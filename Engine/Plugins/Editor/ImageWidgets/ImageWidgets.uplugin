{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "Description": "Generic Slate widgets for displaying images and image-like content, and content related to images.",
  "Category": "Other",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "IsBetaVersion": true,
  "Modules": [
    {
      "Name": "ImageWidgets",
      "Type": "Editor"
    }
  ]
}
