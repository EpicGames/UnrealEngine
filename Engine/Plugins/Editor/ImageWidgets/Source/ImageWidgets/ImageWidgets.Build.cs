// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ImageWidgets : ModuleRules
{
	// Set this flag to true to force building the ColorViewer sample. Otherwise, it will only be built in Debug builds. 
	private const bool ForceBuildColorViewerSample = false;

	public ImageWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			[
				"Core",
				"Slate",
				"SlateCore",
				"UnrealEd"
			]
		);

		PrivateDependencyModuleNames.AddRange(
			[
				"ApplicationCore",
				"CoreUObject",
				"Engine",
				"InputCore"
			]
		);

		bool bBuildColorViewerSample = ForceBuildColorViewerSample || Target.Configuration == UnrealTargetConfiguration.Debug;
		if (bBuildColorViewerSample)
		{
			PrivateDependencyModuleNames.AddRange(
				[
					"Projects",
					"WorkspaceMenuStructure"
				]
			);
		}
		PublicDefinitions.Add(bBuildColorViewerSample ? "IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE=1" : "IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE=0");
	}
}