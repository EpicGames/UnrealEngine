// Copyright Epic Games, Inc. All Rights Reserved.

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "ColorViewer.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"

#define LOCTEXT_NAMESPACE "ColorViewer"

namespace UE::ImageWidgets::Sample
{
namespace Private
{
	static constexpr FGuid InvalidGuid{0, 0, 0, 0}; 

	bool IsValid(const FGuid& Guid)
	{
		return Guid.A != 0;
	}

	void Invalidate(FGuid& Guid)
	{
		Guid.A = 0;
	}
	
	int32 GetIndex(const FGuid& Guid)
	{
		return Guid.B;
	}
}

	IImageViewer::FImageInfo FColorViewer::GetCurrentImageInfo() const
	{
		if (ColorIndexIsValid(SelectedColorIndex) && ColorGuidIsValid(Colors[SelectedColorIndex].Guid))
		{
			return {Colors[SelectedColorIndex].Guid, ImageSize, 0, true};
		}
		return {Private::InvalidGuid, FIntPoint::ZeroValue, 0, false};
	}

	void FColorViewer::DrawCurrentImage(FViewport* Viewport, FCanvas* Canvas, const FDrawProperties& Properties)
	{
		if (Properties.ABComparison.IsActive())
		{
			DrawImage(Properties.ABComparison.GuidA.B, Canvas, Properties.Placement, {0.0, 0.0}, {Properties.ABComparison.Threshold, 1.0});
			DrawImage(Properties.ABComparison.GuidB.B, Canvas, Properties.Placement, {Properties.ABComparison.Threshold, 0.0}, {1.0, 1.0});
		}
		else
		{
			DrawImage(SelectedColorIndex, Canvas, Properties.Placement, {0.0, 0.0}, {1.0, 1.0});
		}
	}

	TOptional<TVariant<FColor, FLinearColor>> FColorViewer::GetCurrentImagePixelColor(FIntPoint PixelCoords, int32 MipLevel) const
	{
		if (ColorIndexIsValid(SelectedColorIndex) && ColorGuidIsValid(Colors[SelectedColorIndex].Guid))
		{
			// Returns the current color.
			// In a less trivial use case, the pixel coordinates and potentially the MIP level would be needed to look up the color value.
			return TVariant<FColor, FLinearColor>(TInPlaceType<FColor>(), Colors[SelectedColorIndex].Color);
		}
		return {};
	}

	void FColorViewer::OnImageSelected(const FGuid& Guid)
	{
		if (ColorGuidIsValid(Guid))
		{
			SelectedColorIndex = Private::GetIndex(Guid);
		}
	}

	bool FColorViewer::IsValidImage(const FGuid& Guid) const
	{
		return ColorGuidIsValid(Guid);
	}

	FText FColorViewer::GetImageName(const FGuid& Guid) const
	{
		if (ColorGuidIsValid(Guid))
		{
			const FColor& Color = Colors[Private::GetIndex(Guid)].Color;
			const FString HexColor = FString::Printf(TEXT("#%02X%02X%02X"), Color.R, Color.G, Color.B);
			return FText::FromString(HexColor);
		}
		return {};
	}

	const FColorViewer::FColorItem* FColorViewer::AddColor()
	{
		Colors.Add({FGuid(1, Colors.Num(), 0, 0), {}, FDateTime::Now()});

		SelectedColorIndex = Colors.Num() - 1;

		RandomizeColor();

		return &Colors[SelectedColorIndex];
	}

	void FColorViewer::RemoveColor(const FGuid& Guid)
	{
		if (ColorGuidIsValid(Guid))
		{
			const int32 Index = Private::GetIndex(Guid);

			Private::Invalidate(Colors[Index].Guid);

			if (SelectedColorIndex == Index)
			{
				SelectedColorIndex = INDEX_NONE;
			}
		}
	}

	const FColorViewer::FColorItem* FColorViewer::RandomizeColor()
	{
		if (ColorIndexIsValid(SelectedColorIndex))
		{
			auto Random = []
			{
				return static_cast<uint8>(FMath::RandRange(0, 255));
			};

			Colors[SelectedColorIndex].Color = {Random(), Random(), Random()};

			return &Colors[SelectedColorIndex];
		}

		return nullptr;
	}

	FToneMapping::EMode FColorViewer::GetToneMapping() const
	{
		return ToneMapping.Mode;
	}

	void FColorViewer::SetToneMapping(FToneMapping::EMode Mode)
	{
		ToneMapping.Mode = Mode;
	}

	FLinearColor FColorViewer::GetDefaultToneMappedColor(const FColor& Color) const
	{
		return ToneMapping.GetToneMappedColor(Color);
	}

	bool FColorViewer::ColorIndexIsValid(int32 Index) const
	{
		return 0 <= Index && Index < Colors.Num();
	}

	bool FColorViewer::ColorGuidIsValid(const FGuid& Guid) const
	{
		return Guid.A != 0 && ColorIndexIsValid(Guid.B) && Colors[Guid.B].Guid == Guid;
	}

	void FColorViewer::DrawImage(int32 Index, FCanvas* Canvas, const FDrawProperties::FPlacement& Placement, const FVector2d& UV0, const FVector2d& UV1) const
	{
		if (ColorIndexIsValid(Index))
		{
			// Get color value after tone mapping.
			const FLinearColor ToneMappedColor = ToneMapping.GetToneMappedColor(Colors[Index].Color);

			// Adjust offset and size based on which part of the image to draw.
			const FVector2d Offset = Placement.Offset + Placement.Size * UV0;
			const FVector2d Size = Placement.Size * (UV1 - UV0);

			// Draw simple quad with current tone mapped color.
			// In a less trivial use case, this would require rendering quads with textures and the like. 
			FCanvasTileItem Tile(Offset, Size, ToneMappedColor);
			Canvas->DrawItem(Tile);
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
