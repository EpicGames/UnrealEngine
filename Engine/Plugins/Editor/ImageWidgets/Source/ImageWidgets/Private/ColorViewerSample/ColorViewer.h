// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "IImageViewer.h"

namespace UE::ImageWidgets::Sample
{
	/**
	 * Provide tone mapping capabilities.
	 * In this simple example, this is limited to just normal RGB plus luminance (grayscale).
	 */
	struct FToneMapping
	{
		enum class EMode { RGB, Lum };

		FToneMapping(EMode Mode)
			: Mode(Mode)
		{
		}

		FLinearColor GetToneMappedColor(const FLinearColor& Color) const
		{
			if (Mode == EMode::Lum)
			{
				const float Luminance = 0.3f * Color.R + 0.59f * Color.G + 0.11f * Color.B;
				return FLinearColor(Luminance, Luminance, Luminance);
			}
			return Color;
		}

		EMode Mode;
	};

	/**
	 * Image viewer implementation used by the image widgets.
	 * It contains any image data, in this case just colors, and renders the image data in the viewport widgets based on the viewport provided parameters.
	 */
	class FColorViewer final : public IImageViewer
	{
	public:
		/**
		 * Necessary data for a color item.
		 */
		struct FColorItem
		{
			/** Unique identifier for each item */
			FGuid Guid;

			/** The actual color value */
			FColor Color;

			/** Timestamp for when the item was created */
			FDateTime DateTime;
		};

		// IImageViewer overrides - begin
		virtual FImageInfo GetCurrentImageInfo() const override;
		virtual void DrawCurrentImage(FViewport* Viewport, FCanvas* Canvas, const FDrawProperties& Properties) override;
		virtual TOptional<TVariant<FColor, FLinearColor>> GetCurrentImagePixelColor(FIntPoint PixelCoords, int32 MipLevel) const override;
		virtual void OnImageSelected(const FGuid& Guid) override;
		virtual bool IsValidImage(const FGuid& Guid) const override;
		virtual FText GetImageName(const FGuid& Guid) const override;
		// IImageViewer overrides - end

		/** Adds a color item. */
		const FColorItem* AddColor();

		/** Removes a color item. */
		void RemoveColor(const FGuid& Guid);
		
		/** Sets a random color as the current "image" as a simple proxy for the image content changing and/or users choosing different images to display. */
		const FColorItem* RandomizeColor();

		/** Access to tone mapping data. This is effectively used by the viewport toolbar extensions as well as when drawing the image. */
		FToneMapping::EMode GetToneMapping() const;
		void SetToneMapping(FToneMapping::EMode Mode);

		/** Apply default tone mapping to a given color. This is used to generate the catalog thumbnail. */
		FLinearColor GetDefaultToneMappedColor(const FColor& Color) const;

		/** Hardcoded values for the image size for all color.
		 *  In a more realistic application, this value would depend on the actual current image. */
		inline static const FIntPoint ImageSize = 512;

	private:
		/** Checks if a given index is a valid image. */
		bool ColorIndexIsValid(int32 Index) const;

		/** Checks if a given guid is a valid image. */
		bool ColorGuidIsValid(const FGuid& Guid) const;

		/** Draws the color image with the given index. The UVs determine if all or only a part of the image is drawn, i.e. for AB comparisons. */
		void DrawImage(int32 Index, FCanvas* Canvas, const FDrawProperties::FPlacement& Placement, const FVector2d& UV0, const FVector2d& UV1) const;
		
		/** The tone mapping data. */
		FToneMapping ToneMapping = FToneMapping::EMode::RGB;

		/** The color for the currently displayed image. */
		TArray<FColorItem> Colors;

		/** Index of the currently selected image. */
		int32 SelectedColorIndex = INDEX_NONE;
	};
}

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
