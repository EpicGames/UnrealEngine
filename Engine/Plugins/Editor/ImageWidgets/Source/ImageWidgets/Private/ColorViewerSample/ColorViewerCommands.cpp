// Copyright Epic Games, Inc. All Rights Reserved.

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "ColorViewerCommands.h"
#include "ColorViewerStyle.h"

#include "Framework/Commands/InputChord.h"

#define LOCTEXT_NAMESPACE "ColorViewerCommands"

namespace UE::ImageWidgets::Sample
{
	FColorViewerCommands::FColorViewerCommands()
		: TCommands("ColorViewer", LOCTEXT("ContextDescription", "Color Viewer"), NAME_None, FColorViewerStyle::Get().GetStyleSetName())
	{
	}

	void FColorViewerCommands::RegisterCommands()
	{
		UI_COMMAND(AddColor, "Add", "Add color to catalog", EUserInterfaceActionType::Button, FInputChord(EKeys::A));
		UI_COMMAND(RandomizeColor, "Randomize", "Set random color", EUserInterfaceActionType::Button, FInputChord(EKeys::R));

		UI_COMMAND(ToneMappingRGB, "RGB", "Show full color", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ToneMappingLum, "Lum", "Show luminance only", EUserInterfaceActionType::ToggleButton, FInputChord());
	}
}

#undef LOCTEXT_NAMESPACE

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
