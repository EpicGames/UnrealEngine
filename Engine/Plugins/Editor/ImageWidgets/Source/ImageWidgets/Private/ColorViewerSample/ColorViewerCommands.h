// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"

namespace UE::ImageWidgets::Sample
{
	/**
	 * Provides commands used for the color viewer sample.
	 */
	class FColorViewerCommands : public TCommands<FColorViewerCommands>
	{
	public:
		FColorViewerCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> AddColor; // Adding a new color entry
		TSharedPtr<FUICommandInfo> RandomizeColor; // Setting current entry to a random color
		TSharedPtr<FUICommandInfo> ToneMappingRGB; // Switching to RGB (no tone mapping)
		TSharedPtr<FUICommandInfo> ToneMappingLum; // Switching to luminance (grayscale tone mapping)
	};
}

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
