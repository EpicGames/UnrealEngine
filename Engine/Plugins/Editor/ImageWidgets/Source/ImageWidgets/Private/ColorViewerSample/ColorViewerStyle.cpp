// Copyright Epic Games, Inc. All Rights Reserved.

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "ColorViewerStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

namespace UE::ImageWidgets::Sample
{
	FName FColorViewerStyle::StyleName("ColorViewerStyle");

	FColorViewerStyle::FColorViewerStyle()
		: FSlateStyleSet(StyleName)
	{
		const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ImageWidgets"))->GetContentDir();

		const FVector2f IconSizeToolbar(16.0f, 16.0f);

		// Use custom icons in the content folder of this plugin.
		Set("ToneMappingRGB", new FSlateImageBrush(ContentDir / TEXT("Icons/ToneMappingRGB") + TEXT(".png"), IconSizeToolbar));
		Set("ToneMappingLum", new FSlateImageBrush(ContentDir / TEXT("Icons/ToneMappingLum") + TEXT(".png"), IconSizeToolbar));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FColorViewerStyle::~FColorViewerStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FColorViewerStyle& FColorViewerStyle::Get()
	{
		static FColorViewerStyle Instance;
		return Instance;
	}
}

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
