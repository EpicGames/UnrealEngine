// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "Styling/SlateStyle.h"

namespace UE::ImageWidgets::Sample
{
	/**
	 * Style declarations for the color viewer sample.
	 */
	class FColorViewerStyle : public FSlateStyleSet
	{
	public:
		static FName StyleName;

		static FColorViewerStyle& Get();

	private:
		FColorViewerStyle();
		virtual ~FColorViewerStyle() override;
	};
}

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE