// Copyright Epic Games, Inc. All Rights Reserved.

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "ColorViewerWidget.h"

#include "ColorViewerCommands.h"
#include "ColorViewerStyle.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "ColorViewerWidget"

namespace UE::ImageWidgets::Sample
{
	void SColorViewerWidget::Construct(const FArguments&)
	{
		ColorViewer = MakeShared<FColorViewer>();

		GroupColors = FName("Colors");
		GroupFavorites = FName("Favorites");

		BindCommands();

		// Create toolbar extensions for a button to randomize the displayed color as well as the tone mapping controls.
		const TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();
		ToolbarExtender->AddToolBarExtension("ToolbarCenter", EExtensionHook::Before, CommandList,
		                                     FToolBarExtensionDelegate::CreateSP(this, &SColorViewerWidget::AddColorButtons));
		ToolbarExtender->AddToolBarExtension("ToolbarRight", EExtensionHook::After, CommandList,
		                                     FToolBarExtensionDelegate::CreateSP(this, &SColorViewerWidget::AddToneMappingButtons));

		// Fill the widget with the image viewport.
		ChildSlot
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(Splitter, SSplitter)
					.PhysicalSplitterHandleSize(2.0f)
					+ SSplitter::Slot()
						.Value(0.2f)
						[
							SAssignNew(Catalog, ImageWidgets::SImageCatalog)
								.DefaultGroupName(GroupColors)
								.DefaultGroupHeading(LOCTEXT("Colors", "Colors"))
								.OnItemSelected_Lambda([&ColorViewer = ColorViewer, &Viewport = Viewport](const FGuid& ImageGuid)
								{
									if (ColorViewer)
									{
										ColorViewer->OnImageSelected(ImageGuid);
									}
									if (Viewport)
									{
										Viewport->RequestRedraw();
									}
								})
								.OnGetGroupContextMenu(this, &SColorViewerWidget::GetGroupContextMenu)
								.OnGetItemsContextMenu(this, &SColorViewerWidget::GetItemsContextMenu)
						]
					+ SSplitter::Slot()
						.Value(0.8f)
						[
							SAssignNew(Viewport, ImageWidgets::SImageViewport, ColorViewer.ToSharedRef())
								.ToolbarExtender(ToolbarExtender)
								.DrawSettings(SImageViewport::FDrawSettings{
									.ClearColor = FLinearColor::Black,
									.bBorderEnabled = true,
									.BorderThickness = 1.0,
									.BorderColor = FVector3f(0.2f),
									.bBackgroundColorEnabled = false,
									.bBackgroundCheckerEnabled = false
								})
								.bABComparisonEnabled(true)
						]
			];

		Catalog->AddGroup(GroupFavorites, LOCTEXT("Favorites", "Favorites"), GroupColors);
	}

	FReply SColorViewerWidget::OnKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
	{
		// Capture all key binds that are handled by the widget's commands. 
		if (CommandList->ProcessCommandBindings(KeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void SColorViewerWidget::AddColorButtons(FToolBarBuilder& ToolbarBuilder) const
	{
		const FSlateIcon AddColorIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Button_Add");
		ToolbarBuilder.AddToolBarButton(FColorViewerCommands::Get().AddColor, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
										TAttribute<FSlateIcon>(AddColorIcon));

		const FSlateIcon RandomizeColorIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Update");
		ToolbarBuilder.AddToolBarButton(FColorViewerCommands::Get().RandomizeColor, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
		                                TAttribute<FSlateIcon>(RandomizeColorIcon));
	}

	void SColorViewerWidget::AddToneMappingButtons(FToolBarBuilder& ToolbarBuilder) const
	{
		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.BeginBlockGroup();
		{
			const FName& StyleSetName = FColorViewerStyle::Get().GetStyleSetName();
			const FColorViewerCommands& Commands = FColorViewerCommands::Get();

			const FSlateIcon RGBIcon(StyleSetName, "ToneMappingRGB");
			const FSlateIcon LumIcon(StyleSetName, "ToneMappingLum");

			ToolbarBuilder.AddToolBarButton(Commands.ToneMappingRGB, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(RGBIcon));
			ToolbarBuilder.AddToolBarButton(Commands.ToneMappingLum, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(LumIcon));
		}
		ToolbarBuilder.EndBlockGroup();
	}

	TTuple<FText, FText, FText> GetColorItemMetaData(const FColorViewer& ColorViewer, const FColorViewer::FColorItem* ColorItem)
	{
		const FText Name = ColorViewer.GetImageName(ColorItem->Guid);

		const FText Info = FText::Format(
			LOCTEXT("ColorEntryInfoLabel", "{0}"), FText::AsTime(ColorItem->DateTime, EDateTimeStyle::Short, FText::GetInvariantTimeZone()));

		const FText ToolTip = FText::Format(
			LOCTEXT("ColorEntryToolTip", "R {0}, G {1}, B {2}"), {ColorItem->Color.R, ColorItem->Color.G, ColorItem->Color.B});

		return {Name, Info, ToolTip};
	}

	void SColorViewerWidget::AddColor()
	{
		if (const FColorViewer::FColorItem* ColorItem = ColorViewer->AddColor())
		{
			auto [Name, Info, ToolTip] = GetColorItemMetaData(*ColorViewer, ColorItem);
			const FLinearColor ToneMappedColor = ColorViewer->GetDefaultToneMappedColor(ColorItem->Color);

			Catalog->AddItem(MakeShared<FImageCatalogItemData>(ColorItem->Guid, FSlateColorBrush(ToneMappedColor), Name, Info, ToolTip));
			Catalog->ClearSelection();
			Catalog->SelectItem(ColorItem->Guid);
		}
	}

	void SColorViewerWidget::RandomizeColor()
	{
		if (const FColorViewer::FColorItem* ColorItem = ColorViewer->RandomizeColor())
		{
			const auto [Name, Info, ToolTip] = GetColorItemMetaData(*ColorViewer, ColorItem);
			const FLinearColor ToneMappedColor = ColorViewer->GetDefaultToneMappedColor(ColorItem->Color);

			Catalog->UpdateItem({ColorItem->Guid, FSlateColorBrush(ToneMappedColor), Name, Info, ToolTip});
		}
	}

	bool SColorViewerWidget::RandomizeColorEnabled() const
	{
		const IImageViewer::FImageInfo Info = ColorViewer->GetCurrentImageInfo();
		if (Info.bIsValid)
		{
			if (const TOptional<FName> GroupName = Catalog->GetItemGroupName(Info.Guid))
			{
				if (*GroupName == GroupColors)
				{
					return true;
				}
			}
		}

		return false;
	}

	TSharedPtr<SWidget> SColorViewerWidget::GetGroupContextMenu(FName GroupName) const
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		const int32 NumItems = Catalog->NumItems(GroupName);
		if (NumItems > 0)
		{
			MenuBuilder.AddMenuEntry(
				NumItems == 1
					? FText::Format(LOCTEXT("DeleteGroupItem", "Delete single item in group"), NumItems)
					: FText::Format(LOCTEXT("DeleteAllGroupItems", "Delete all {0} items in group"), NumItems),
				LOCTEXT("DeleteGroupItems", "Deletes all items in this group."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
				FUIAction(FExecuteAction::CreateLambda(
					[GroupName, Catalog = Catalog, ColorViewer = ColorViewer]
					{
						if (Catalog && ColorViewer)
						{
							for (int32 Index = Catalog->NumItems(GroupName) - 1; Index >= 0; --Index)
							{
								const TOptional<FGuid> Guid = Catalog->GetItemGuidAt(Index, GroupName);
								if (Guid.IsSet())
								{
									Catalog->RemoveItem(Guid.GetValue());
									ColorViewer->RemoveColor(Guid.GetValue());
								}
							}
						}
					})));
		}

		return MenuBuilder.MakeWidget();
	}

	TSharedPtr<SWidget> SColorViewerWidget::GetItemsContextMenu(const TArray<FGuid>& Guids) const
	{
		checkSlow(!Guids.IsEmpty());

		auto IsFavorite = [&Catalog = Catalog, &GroupFavorites = GroupFavorites](const FGuid& Guid)
		{
			const TOptional<FName> GroupName = Catalog->GetItemGroupName(Guid);
			return GroupName && *GroupName == GroupFavorites;
		};

		const bool bHaveFavorites = Algo::AnyOf(Guids, IsFavorite);
		const bool bHaveNonFavorites = !Algo::AllOf(Guids, IsFavorite);

		FMenuBuilder MenuBuilder(true, nullptr);

		if (bHaveNonFavorites)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddFavorite", "Add To Favorites"),
				LOCTEXT("AddFavoriteTooltip", "Adds the selected color(s) to the list of favorites."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Pinned"),
				FUIAction(
					FExecuteAction::CreateLambda(
						[Guids, Catalog = Catalog, GroupFavorites = GroupFavorites, GroupColors = GroupColors]
						{
							if (Catalog)
							{
								for (const FGuid& Guid : Guids)
								{
									const auto GroupName = Catalog->GetItemGroupName(Guid);
									if (GroupName && *GroupName == GroupColors)
									{
										Catalog->MoveItem(Guid, GroupFavorites);
									}
									Catalog->SelectItem(Guid);
								}
							}
						})));
		}

		if (bHaveFavorites)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveFavorite", "Remove from Favorites"),
				LOCTEXT("RemoveFavoriteTooltip", "Removes the selected color(s) from the list of favorites."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unpinned"),
				FUIAction(
					FExecuteAction::CreateLambda(
						[Guids, Catalog = Catalog, GroupFavorites = GroupFavorites, GroupColors = GroupColors]
						{
							if (Catalog)
							{
								for (const FGuid& Guid : Guids)
								{
									const auto GroupName = Catalog->GetItemGroupName(Guid);
									if (GroupName && *GroupName == GroupFavorites)
									{
										Catalog->MoveItem(Guid, GroupColors);
									}
									Catalog->SelectItem(Guid);
								}
							}
						})));
		}

		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteColors", "Delete"),
			LOCTEXT("DeleteColorsTooltip", "Deletes the selected color(s) from the catalog."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(FExecuteAction::CreateLambda(
				[Guids, Catalog = Catalog, ColorViewer = ColorViewer]
				{
					if (Catalog && ColorViewer)
					{
						for (const FGuid& Guid : Guids)
						{
							Catalog->RemoveItem(Guid);
							ColorViewer->RemoveColor(Guid);
						}
					}
				})));

		return MenuBuilder.MakeWidget();
	}

	void SColorViewerWidget::BindCommands()
	{
		const FColorViewerCommands& Commands = FColorViewerCommands::Get();

		CommandList = MakeShared<FUICommandList>();

		CommandList->MapAction(
			Commands.AddColor,
			FExecuteAction::CreateSP(this, &SColorViewerWidget::AddColor)
		);

		CommandList->MapAction(
			Commands.RandomizeColor,
			FExecuteAction::CreateSP(this, &SColorViewerWidget::RandomizeColor),
			FCanExecuteAction::CreateSP(this, &SColorViewerWidget::RandomizeColorEnabled)
		);

		CommandList->MapAction(
			Commands.ToneMappingRGB,
			FExecuteAction::CreateSP(ColorViewer.ToSharedRef(), &FColorViewer::SetToneMapping, FToneMapping::EMode::RGB),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([ColorViewer = ColorViewer]
			{
				return ColorViewer->GetToneMapping() == FToneMapping::EMode::RGB;
			})
		);

		CommandList->MapAction(
			Commands.ToneMappingLum,
			FExecuteAction::CreateSP(ColorViewer.ToSharedRef(), &FColorViewer::SetToneMapping, FToneMapping::EMode::Lum),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([ColorViewer = ColorViewer]
			{
				return ColorViewer->GetToneMapping() == FToneMapping::EMode::Lum;
			})
		);
	}
}

#undef LOCTEXT_NAMESPACE

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
