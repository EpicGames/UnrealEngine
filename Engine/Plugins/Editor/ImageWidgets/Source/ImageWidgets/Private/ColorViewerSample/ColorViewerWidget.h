// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "ColorViewer.h"

#include "SImageCatalog.h"
#include "SImageViewport.h"
#include "Widgets/Layout/SSplitter.h"

namespace UE::ImageWidgets::Sample
{
	/**
	 * Widget that contains and configures the image widgets.
	 */
	class SColorViewerWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SColorViewerWidget)
			{
			}

		SLATE_END_ARGS()

		/** Gets called by Slate for construction of this widget. */
		void Construct(const FArguments& Args);

		/** Forwards key presses to the command bindings. */
		virtual FReply OnKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent) override;

	private:
		/** Binds all the commands. */
		void BindCommands();

		/** Functions for adding buttons as viewport toolbar extensions. */
		void AddColorButtons(FToolBarBuilder& ToolbarBuilder) const;
		void AddToneMappingButtons(FToolBarBuilder& ToolbarBuilder) const;

		/** Add a new color entry to the catalog. */
		void AddColor();

		/** Choose a random color for the current entry. */
		void RandomizeColor();

		/** Checks if the Randomize Color action is available. */
		bool RandomizeColorEnabled() const;

		/** Creates a context menu for a group. */
		TSharedPtr<SWidget> GetGroupContextMenu(FName GroupName) const;

		/** Creates a context menu for a set of selected guids. */
		TSharedPtr<SWidget> GetItemsContextMenu(const TArray<FGuid>& Guids) const;

		/** The image viewer implementation that contains the image data and renders the image. */
		TSharedPtr<FColorViewer> ColorViewer;

		/** Adjustable divider between catalog on the left and viewport on the right. */
		TSharedPtr<SSplitter> Splitter;

		/** The image catalog that holds all currently available images. */
		TSharedPtr<SImageCatalog> Catalog;

		/** The image viewport in which the current image gets displayed in. */
		TSharedPtr<SImageViewport> Viewport;

		/** The commands used by this sample widget. */
		TSharedPtr<FUICommandList> CommandList;

		/** Names for groups in the catalog. */
		FName GroupColors;
		FName GroupFavorites;
	};
}

#endif // IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
