// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"

namespace UE::ImageWidgets
{
	/**
	 * Provides data and logic for AB comparisons of images.
	 */
	class FImageABComparison
	{
	public:
		DECLARE_DELEGATE_RetVal(FGuid, FGetCurrentImageGuid)
		DECLARE_DELEGATE_RetVal_OneParam(bool, FImageIsValid, const FGuid&)
		DECLARE_DELEGATE_RetVal_OneParam(FText, FGetImageName, const FGuid&)

		FImageABComparison(FImageIsValid&& InImageIsValid, FGetCurrentImageGuid&& InGetCurrentImageGuid, FGetImageName&& InGetImageName)
			: ImageIsValid(MoveTemp(InImageIsValid))
			, GetCurrentImageGuid(MoveTemp(InGetCurrentImageGuid))
			, GetImageName(MoveTemp(InGetImageName))
		{
			check(ImageIsValid.IsBound());
			check(GetCurrentImageGuid.IsBound());
			check(GetImageName.IsBound());
		}

		enum class EAorB : int8
		{
			A,
			B
		};

		bool CanSetABComparison(EAorB AorB) const
		{
			const FGuid CurrentGuid = GetCurrentImageGuid.Execute();
			const bool bGuidIsValidAndCurrentGuid = Guids[static_cast<int32>(AorB)].IsValid() && Guids[static_cast<int32>(AorB)] == CurrentGuid;
			const bool bViewerHasValidImage = ImageIsValid.Execute(CurrentGuid);
			const bool bGuidIsDisabledAndOtherGuidIsNotCurrentGuid = !Guids[static_cast<int32>(AorB)].IsValid() && Guids[!static_cast<bool>(AorB)] != CurrentGuid;

			return bGuidIsValidAndCurrentGuid || (bViewerHasValidImage && bGuidIsDisabledAndOtherGuidIsNotCurrentGuid);
		}

		void SetABComparison(EAorB AorB, const FGuid& Guid)
		{
			Guids[static_cast<int32>(AorB)] = Guid;
		}

		bool ABComparisonIsSet(EAorB AorB) const
		{
			return Guids[static_cast<int32>(AorB)].IsValid();
		}

		bool IsActive() const
		{
			return Guids[static_cast<int32>(EAorB::A)].IsValid() && Guids[static_cast<int32>(EAorB::B)].IsValid();
		}

		const FGuid& GetGuidA() const
		{
			return Guids[static_cast<int32>(EAorB::A)];
		}

		const FGuid& GetGuidB() const
		{
			return Guids[static_cast<int32>(EAorB::B)];
		}

		FText GetName(EAorB AorB) const
		{
			return Guids[static_cast<int32>(AorB)].IsValid() ? GetImageName.Execute(Guids[static_cast<int32>(AorB)]) : FText();
		}

	private:
		FGuid Guids[2];
		FImageIsValid ImageIsValid;
		FGetCurrentImageGuid GetCurrentImageGuid;
		FGetImageName GetImageName;
	};
}
