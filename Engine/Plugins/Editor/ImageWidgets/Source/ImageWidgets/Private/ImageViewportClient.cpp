// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewportClient.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "ImageABComparison.h"
#include "ImageWidgetsLogCategory.h"
#include "SImageViewport.h"
#include "Texture2DPreview.h"
#include "TextureResource.h"
#include "Engine/Texture2D.h"

#define LOCTEXT_NAMESPACE "ImageViewportClient"

namespace UE::ImageWidgets
{

UTexture2D* CreateCheckerTexture(const FColor& Color1, const FColor& Color2, const int32 CheckerSize)
{
	constexpr EPixelFormat Format = PF_B8G8R8A8;
	const int32 TextureSize = FMath::Min<int32>(FMath::RoundUpToPowerOfTwo(FMath::Max(GPixelFormats[PF_B8G8R8A8].BlockSizeX, CheckerSize)) * 2, 4096);

	UTexture2D* CheckerTexture = UTexture2D::CreateTransient(TextureSize, TextureSize, Format);
	check(CheckerTexture != nullptr);

	UE_LOGF(LogImageWidgets, Verbose, "Creating background checker texture '%ls' with Color1=%ls, Color2=%ls, and Size=%d.",
	       *CheckerTexture->GetName(), *Color1.ToHex(), *Color2.ToHex(), CheckerSize);

	// TODO Revisit setting the LODGroup, and figure out if there is a better way to do this.
	// Setting the LOD group seems to be the only way to avoid interpolation when zoomed in. 
	CheckerTexture->LODGroup = TEXTUREGROUP_Pixels2D;

	TIndirectArray<FTexture2DMipMap>& Mips = CheckerTexture->GetPlatformData()->Mips;
	int32 MipSize = TextureSize;
	while (MipSize >= GPixelFormats[Format].BlockSizeX)
	{
		const int32 MipIndex = MipSize == TextureSize ? 0 : Mips.Add(new FTexture2DMipMap(MipSize, MipSize, 1));

		FByteBulkData& MipBulkData = Mips[MipIndex].BulkData;

		FColor *const MipColorData = [&MipBulkData, MipIndex, MipSize]
		{
			FColor* ColorData = static_cast<FColor*>(MipBulkData.Lock(LOCK_READ_WRITE));
			if (MipIndex > 0)
			{
				const int64 NumBlocks = MipSize / GPixelFormats[Format].BlockSizeX;
				const int64 NumBytes = NumBlocks * NumBlocks * GPixelFormats[Format].BlockBytes;

				ColorData = reinterpret_cast<FColor*>(MipBulkData.Realloc(NumBytes));
			}
			return ColorData;
		}();

		if (MipSize > 1)
		{
			const int32 MipCheckerSize = MipSize / 2;
			for (int32 Y = 0; Y < MipSize; ++Y)
			{
				const bool bTop = Y < MipCheckerSize;
				for (int32 X = 0; X < MipSize; ++X)
				{
					const bool bLeft = X < MipCheckerSize;
					MipColorData[X + Y * MipSize] = bTop == bLeft ? Color1 : Color2;
				}
			}
		}
		else
		{
			*MipColorData = FColor((static_cast<uint16>(Color1.R) + Color2.R) / 2,
			                       (static_cast<uint16>(Color1.G) + Color2.G) / 2,
			                       (static_cast<uint16>(Color1.B) + Color2.B) / 2);
		}

		MipBulkData.Unlock();

		MipSize /= 2;
	}

	CheckerTexture->UpdateResource();

	return CheckerTexture;
}

void DestroyCheckerTexture(TStrongObjectPtr<UTexture2D>& CheckerTexture)
{
	if (CheckerTexture)
	{
		UE_LOGF(LogImageWidgets, Verbose, "Destroying background checker texture '%ls'.", *CheckerTexture->GetName());

		if (CheckerTexture->GetResource())
		{
			CheckerTexture->ReleaseResource();
		}
		CheckerTexture->MarkAsGarbage();

		CheckerTexture = nullptr;
	}
}

FImageViewportClient::FImageViewportClient(const TWeakPtr<SEditorViewport>& InEditorViewport, FGetImageSize&& InGetImageSize, FDrawImage&& InDrawImage,
                                           FGetDrawSettings&& InGetDrawSettings, FGetLayoutAndDPIScaleFactor&& InGetLayoutAndDPIScaleFactor,
                                           FGetLayoutScaleFactor&& InGetLayoutScaleFactor, const FImageABComparison* InABComparison,
                                           const SImageViewport::FControllerSettings& InControllerSettings)
	: FEditorViewportClient(nullptr, nullptr, InEditorViewport)
	, GetImageSize(MoveTemp(InGetImageSize))
	, DrawImage(MoveTemp(InDrawImage))
	, GetDrawSettings(MoveTemp(InGetDrawSettings))
	, GetLayoutAndDPIScaleFactor(MoveTemp(InGetLayoutAndDPIScaleFactor))
	, GetLayoutScaleFactor(MoveTemp(InGetLayoutScaleFactor))
	, OnInputKey(InControllerSettings.OnInputKey)
	, ABComparison(InABComparison)
	, Controller(static_cast<FImageViewportController::EZoomMode>(InControllerSettings.DefaultZoomMode))
	, bZoomOnResize(InControllerSettings.bZoomOnResize)
{
	check(GetImageSize.IsBound());
	check(DrawImage.IsBound());
	check(GetDrawSettings.IsBound());
	check(GetLayoutAndDPIScaleFactor.IsBound());
	check(GetLayoutScaleFactor.IsBound());
	check(ABComparison);

	SetRealtime(true);
}

FImageViewportClient::~FImageViewportClient()
{
	if (CheckerTexture)
	{
		DestroyCheckerTexture(CheckerTexture);
	}
}

void FImageViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	const SImageViewport::FDrawSettings DrawSettings = GetDrawSettings.Execute();
	CreateOrDestroyCheckerTextureIfSettingsChanged(DrawSettings);

	Canvas->Clear(DrawSettings.ClearColor);

	const FIntPoint ImageSize = GetImageSize.Execute();
	if (ImageSize.X <= 0 || ImageSize.Y <= 0)
	{
		return;
	}

	const float LayoutAndDPIScale = GetLayoutAndDPIScaleFactor.Execute();

	// Recalculate zoom and pan when the viewport is resized.
	{
		const FVector2d ScaledSize = FVector2d(InViewport->GetSizeXY()) / LayoutAndDPIScale;
		if (ScaledSize != CachedScaledViewportSize)
		{
			if (CachedScaledViewportSize != FVector2d::ZeroVector)
			{
				const FImageViewportController::FZoomSettings OldZoomSettings = Controller.GetZoom();
				const FVector2d OldPan = Controller.GetPan(FVector2d::Zero());

				if (OldZoomSettings.Mode != FImageViewportController::EZoomMode::Custom)
				{
					// Fill/Fit: recalculate zoom for new viewport size.
					Controller.SetZoom(OldZoomSettings.Mode, OldZoomSettings.Zoom, ImageSize, ScaledSize);
				}
				else if (bZoomOnResize)
				{
					// When zoom on resize is set, scale zoom proportionally.
					const double OldFillZoom = (CachedScaledViewportSize / FVector2d(ImageSize)).GetMin();
					const double NewFillZoom = (ScaledSize / FVector2d(ImageSize)).GetMin();
					if (OldFillZoom > SMALL_NUMBER)
					{
						Controller.SetZoom(FImageViewportController::EZoomMode::Custom, OldZoomSettings.Zoom * (NewFillZoom / OldFillZoom),
						                   ImageSize, ScaledSize);
					}
				}

				// Scale pan uniformly by the zoom ratio to maintain relative image position.
				// Using a single scalar (NewZoom/OldZoom) for both axes avoids diagonal drift when the constraining axis flips, e.g. landscape to portrait.
				const double NewZoom = Controller.GetZoom().Zoom;
				const double PanScale = OldZoomSettings.Zoom > SMALL_NUMBER ? NewZoom / OldZoomSettings.Zoom : 1.0;
				const FVector2d NewPan = OldPan * PanScale;
				const FVector2d CurrentPan = Controller.GetPan(FVector2d::Zero());
				Controller.Pan(NewPan - CurrentPan);
			}

			CachedScaledViewportSize = ScaledSize;
		}
	}

	const float LayoutScale = GetLayoutScaleFactor.Execute();

	// Controller computes placement in unzoomed logical space
	const FVector2d ScaledViewportSize = FVector2d(InViewport->GetSizeXY()) / LayoutAndDPIScale;
	CachedPlacement = GetPlacementProperties(ImageSize, ScaledViewportSize);
	bCachedPlacementIsValid = true;

	const IImageViewer::FDrawProperties::FMip MipProperties = GetMipProperties();

	// Scale placement from unzoomed logical space to RT pixel space for drawing
	const FVector2d DrawOffset = CachedPlacement.Offset * LayoutScale;
	const FVector2d DrawSize = CachedPlacement.Size * LayoutScale;

	// Draw border
	if (DrawSettings.bBorderEnabled)
	{
		FCanvasBoxItem Border(DrawOffset - DrawSettings.BorderThickness / 2.0, DrawSize + DrawSettings.BorderThickness);
		Border.LineThickness = DrawSettings.BorderThickness;
		Border.SetColor(DrawSettings.BorderColor);
		Canvas->DrawItem(Border);
	}

	// Draw background
	if (DrawSettings.bBackgroundColorEnabled || DrawSettings.bBackgroundCheckerEnabled)
	{
		FCanvasTileItem Background(DrawOffset, DrawSize, DrawSettings.BackgroundColor);

		TRefCountPtr<FBatchedElementTexture2DPreviewParameters> BatchedElementParameters;

		if (DrawSettings.bBackgroundCheckerEnabled)
		{
			if (ensure(CheckerTexture != nullptr))
			{
				Background.SetColor(FLinearColor::White);
				Background.Texture = CheckerTexture->GetResource();

				Background.UV1 = FVector2D(ImageSize) / CheckerTexture->GetSizeX();

				BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(
					-1, 0, 0, false, false, false, false, false, false);
				Background.BatchedElementParameters = BatchedElementParameters.GetReference();
			}
		}

		Canvas->DrawItem(Background);
	}

	const double ABComparisonDividerX = [this, LayoutScale, &ImageSize]
	{
		if (bDraggingABComparisonDivider)
		{
			FIntPoint MousePos;
			Viewport->GetMousePos(MousePos);

			// Mouse pos is in RT pixel space; convert to unzoomed placement space
			FVector2d DividerPos = FVector2d(MousePos) / LayoutScale - CachedPlacement.Offset;
			DividerPos /= CachedPlacement.ZoomFactor;

			return DividerPos.X / ImageSize.X;
		}
		return ABComparisonDivider;
	}();

	const IImageViewer::FDrawProperties::FABComparison ABComparisonProperties = [this, &ABComparisonDividerX]() -> IImageViewer::FDrawProperties::FABComparison
	{
		return {ABComparison->GetGuidA(), ABComparison->GetGuidB(), FMath::Clamp(ABComparisonDividerX, 0.0, 1.0)};
	}();

	// Draw image — pass scaled placement for RT pixel space drawing
	const IImageViewer::FDrawProperties::FPlacement DrawPlacement{DrawOffset, DrawSize, CachedPlacement.ZoomFactor * LayoutScale};
	DrawImage.Execute(InViewport, Canvas, {DrawPlacement, MipProperties, ABComparisonProperties});

	if (ABComparison->IsActive())
	{
		const FVector2D LineStart(DrawOffset.X + DrawSize.X * ABComparisonDividerX, 0);
		const FVector2D LineEnd(DrawOffset.X + DrawSize.X * ABComparisonDividerX, InViewport->GetSizeXY().Y);
		FCanvasLineItem Line(LineStart, LineEnd);
		Line.LineThickness = 2.0f;
		Line.SetColor({0.5f, 0.5f, 0.5f, 1.0f});
		Canvas->DrawItem(Line);
	}
}

EMouseCursor::Type FImageViewportClient::GetCursor(FViewport* InViewport, int32 X, int32 Y)	
{
	if (bDragging)
	{
		CachedMouseX = X;
		CachedMouseY = Y;
		return EMouseCursor::GrabHandClosed;
	}

	if (bDraggingABComparisonDivider || MouseIsOverABComparisonDivider({X, Y}))
	{
		return EMouseCursor::ResizeLeftRight;
	}

	const auto [PixelCoordsValid, PixelCoords] = GetPixelCoordinatesUnderCursor();
	const FIntPoint ImageSize = GetImageSize.Execute();
	if (PixelCoordsValid && 0 <= PixelCoords.X && 0 <= PixelCoords.Y && PixelCoords.X < ImageSize.X && PixelCoords.Y < ImageSize.Y)
	{
		return EMouseCursor::Crosshairs;
	}

	return EMouseCursor::Default;
}

bool FImageViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	if (EventArgs.Event == IE_Pressed)
	{
		if (EventArgs.Key == EKeys::MouseScrollUp || EventArgs.Key == EKeys::Add)
		{
			const auto [PixelCoordsValid, PixelCoords] = GetPixelCoordinatesUnderCursor();
			if (PixelCoordsValid)
			{
				Controller.ZoomIn(PixelCoords, GetImageSize.Execute());
			}
			return true;
		}

		if (EventArgs.Key == EKeys::MouseScrollDown || EventArgs.Key == EKeys::Subtract)
		{
			const auto [PixelCoordsValid, PixelCoords] = GetPixelCoordinatesUnderCursor();
			if (PixelCoordsValid)
			{
				Controller.ZoomOut(PixelCoords, GetImageSize.Execute());
			}
			return true;
		}

		if (EventArgs.Key == EKeys::F)
		{
			Controller.Reset(GetImageSize.Execute(), GetScaledViewportSize());

			return true;
		}
	}

	if (OnInputKey.IsBound())
	{
		if (OnInputKey.Execute(EventArgs))
		{
			return true;
		}
	}

	return FEditorViewportClient::InputKey(EventArgs);
}

void FImageViewportClient::TrackingStarted(const FInputEventState& InputState, bool bIsDraggingWidget, bool bNudge)
{
	if (!bIsDraggingWidget)
	{
		FIntPoint MousePos;
		InputState.GetViewport()->GetMousePos(MousePos);

		if (InputState.IsLeftMouseButtonPressed() && MouseIsOverABComparisonDivider(MousePos))
		{
			bDraggingABComparisonDivider = true;
			return;
		}

		if (InputState.IsRightMouseButtonPressed())
		{
			bDragging = true;
			DraggingStart = MousePos;
			return;
		}
	}

	FEditorViewportClient::TrackingStarted(InputState, bIsDraggingWidget, bNudge);
}

void FImageViewportClient::TrackingStopped()
{
	if (bDraggingABComparisonDivider)
	{
		bDraggingABComparisonDivider = false;

		FIntPoint DraggingEnd;
		Viewport->GetMousePos(DraggingEnd);

		const float LayoutScale = GetLayoutScaleFactor.Execute();

		// Mouse pos is in RT pixel space; convert to unzoomed placement space
		FVector2d DraggingEndPos = FVector2d(DraggingEnd) / LayoutScale - CachedPlacement.Offset;
		DraggingEndPos /= CachedPlacement.ZoomFactor;

		const FIntPoint ImageSize = GetImageSize.Execute();
		if (ImageSize.X > 0)
		{
			ABComparisonDivider = DraggingEndPos.X / ImageSize.X;
		}
	}

	if (bDragging)
	{
		bDragging = false;

		FIntPoint DraggingEnd;
		Viewport->GetMousePos(DraggingEnd);

		// Drag delta is in RT pixel space; divide by layout and DPI scale to get unzoomed logical space
		Controller.Pan(FVector2d(DraggingEnd - DraggingStart) / GetLayoutAndDPIScaleFactor.Execute());
	}

	RequiredCursorVisibiltyAndAppearance.bDontResetCursor = true;

	FEditorViewportClient::TrackingStopped();
}

int32 FImageViewportClient::GetMipLevel() const
{
	return MipLevel;
}

void FImageViewportClient::SetMipLevel(int32 InMipLevel)
{
	MipLevel = InMipLevel;
}

void FImageViewportClient::ResetController(FIntPoint ImageSize)
{
	Controller.Reset(ImageSize, GetScaledViewportSize());
}

void FImageViewportClient::ResetZoom(FIntPoint ImageSize)
{
	const FImageViewportController::FZoomSettings ZoomSettings = Controller.GetZoom();
	Controller.SetZoom(ZoomSettings.Mode, ZoomSettings.Zoom, ImageSize, GetScaledViewportSize());
}

FImageViewportController::FZoomSettings FImageViewportClient::GetZoom() const
{
	FImageViewportController::FZoomSettings ZoomSettings = Controller.GetZoom();
	ZoomSettings.Zoom *= GetLayoutAndDPIScaleFactor.Execute();
	return ZoomSettings;
}

void FImageViewportClient::SetZoom(FImageViewportController::EZoomMode Mode, double Zoom)
{
	Controller.SetZoom(Mode, Zoom, GetImageSize.Execute(), GetScaledViewportSize());
}

TPair<bool, FVector2d> FImageViewportClient::GetPixelCoordinatesUnderCursor() const
{
	if (!bCachedPlacementIsValid || CurrentMousePos == FIntPoint{-1, -1})
	{
		return {false, FVector2d::Zero()};
	}

	const float LayoutAndDPIScale = GetLayoutAndDPIScaleFactor.Execute();

	// Mouse pos is in RT pixel space; convert to unzoomed logical space (where CachedPlacement lives)
	const FVector2d MousePos((CurrentMousePos.X + 0.5) / LayoutAndDPIScale, (CurrentMousePos.Y + 0.5) / LayoutAndDPIScale);
	const FVector2d CurrentDrag = GetCurrentDrag();

	FVector2d CursorPos = FVector2d(MousePos) - CachedPlacement.Offset + CurrentDrag;
	CursorPos /= CachedPlacement.ZoomFactor;

	return {true, CursorPos};
}

FVector2d FImageViewportClient::GetCurrentDrag() const
{
	if (bDragging)
	{
		FIntPoint DraggingEnd;
		Viewport->GetMousePos(DraggingEnd);

		// Drag delta is in RT pixel space; convert to unzoomed logical space
		return FVector2d(DraggingEnd - DraggingStart) / GetLayoutAndDPIScaleFactor.Execute();
	}
	return FVector2d::Zero();
}

IImageViewer::FDrawProperties::FPlacement FImageViewportClient::GetPlacementProperties(const FIntPoint ImageSize,
                                                                                       const FVector2d ScaledViewportSize) const
{
	const FVector2d CurrentDrag = GetCurrentDrag();
	const FVector2d Pan = Controller.GetPan(CurrentDrag);

	const double ZoomFactor = Controller.GetZoom().Zoom;

	const FVector2d TileSize = FVector2d(ImageSize) * ZoomFactor;
	const FVector2d TileOffset = (ScaledViewportSize - TileSize) / 2 + Pan;

	return {TileOffset, TileSize, ZoomFactor};
}

IImageViewer::FDrawProperties::FMip FImageViewportClient::GetMipProperties() const
{
	const double ZoomFactor = Controller.GetZoom().Zoom;
	const float MipFactor = 1.0f / FMath::Pow(2.0f, MipLevel);
	const float EffectiveMipLevel = ZoomFactor < MipFactor ? -1.0f : MipLevel;

	return {EffectiveMipLevel};
}

void FImageViewportClient::CreateOrDestroyCheckerTextureIfSettingsChanged(const SImageViewport::FDrawSettings& DrawSettings)
{
	const FCheckerTextureSettings NewCheckerTextureSettings{
		DrawSettings.bBackgroundCheckerEnabled, DrawSettings.BackgroundCheckerColor1, DrawSettings.BackgroundCheckerColor2, DrawSettings.BackgroundCheckerSize
	};

	if (CachedCheckerTextureSettings != NewCheckerTextureSettings)
	{
		if (CheckerTexture)
		{
		    DestroyCheckerTexture(CheckerTexture);
        }

        if (NewCheckerTextureSettings.bEnabled)
		{
			CheckerTexture = TStrongObjectPtr<UTexture2D>(CreateCheckerTexture(NewCheckerTextureSettings.Color1.ToFColorSRGB(),
			                                                                   NewCheckerTextureSettings.Color2.ToFColorSRGB(),
			                                                                   NewCheckerTextureSettings.CheckerSize));
		}

		CachedCheckerTextureSettings = NewCheckerTextureSettings;
	}
}

FVector2d FImageViewportClient::GetScaledViewportSize() const
{
	// Return unzoomed logical size for controller Reset/SetZoom calls
	return FVector2d(Viewport->GetSizeXY()) / GetLayoutAndDPIScaleFactor.Execute();
}

bool FImageViewportClient::MouseIsOverABComparisonDivider(const FIntPoint MousePos) const
{
	// Scale the divider position from unzoomed logical space to RT pixel space for comparison with mouse pos
	const float LayoutScale = GetLayoutScaleFactor.Execute();
	const double ABComparisonDividerPosition = (CachedPlacement.Offset.X + CachedPlacement.Size.X * ABComparisonDivider) * LayoutScale;
	const double HitMargin = FMath::Max(4.0, 2.0 * LayoutScale);
	return ABComparisonDividerPosition - HitMargin <= MousePos.X && MousePos.X <= ABComparisonDividerPosition + HitMargin;
}
}

#undef LOCTEXT_NAMESPACE
