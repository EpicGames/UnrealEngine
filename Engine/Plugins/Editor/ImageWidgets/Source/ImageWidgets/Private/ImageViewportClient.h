// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "ImageViewportController.h"
#include "IImageViewer.h"
#include "SImageViewport.h"

namespace UE::ImageWidgets
{
	class FImageABComparison;

	DECLARE_DELEGATE_RetVal(FIntPoint, FGetImageSize)
	DECLARE_DELEGATE_ThreeParams(FDrawImage, FViewport*, FCanvas*, const IImageViewer::FDrawProperties&)
	DECLARE_DELEGATE_RetVal(SImageViewport::FDrawSettings, FGetDrawSettings)
	DECLARE_DELEGATE_RetVal(float, FGetLayoutAndDPIScaleFactor)
	DECLARE_DELEGATE_RetVal(float, FGetLayoutScaleFactor)

	/**
	 * Viewport client for controlling the camera and drawing viewport contents. 
	 */
	class FImageViewportClient : public FEditorViewportClient
	{
	public:
		FImageViewportClient(const TWeakPtr<SEditorViewport>& InEditorViewport, FGetImageSize&& InGetImageSize, FDrawImage&& InDrawImage,
		                     FGetDrawSettings&& InGetDrawSettings, FGetLayoutAndDPIScaleFactor&& InGetLayoutAndDPIScaleFactor,
		                     FGetLayoutScaleFactor&& InGetLayoutScaleFactor, const FImageABComparison* InABComparison,
		                     const SImageViewport::FControllerSettings& InControllerSettings);
		virtual ~FImageViewportClient() override;

		virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override;

		// FEditorViewportClient overrides - begin
		virtual EMouseCursor::Type GetCursor(FViewport* InViewport, int32 X, int32 Y) override;
		virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
		virtual void TrackingStarted(const FInputEventState& InputState, bool bIsDraggingWidget, bool bNudge) override;
		virtual void TrackingStopped() override;
		// FEditorViewportClient overrides - end

		int32 GetMipLevel() const;
		void SetMipLevel(int32 InMipLevel);
		FImageViewportController::FZoomSettings GetZoom() const;
		void SetZoom(FImageViewportController::EZoomMode Mode, double Zoom);
		void ResetController(FIntPoint ImageSize);
		void ResetZoom(FIntPoint ImageSize);

		TPair<bool, FVector2d> GetPixelCoordinatesUnderCursor() const;

	private:
		struct FCheckerTextureSettings
		{
			bool operator==(const FCheckerTextureSettings&) const = default;

			bool bEnabled = false;
			FLinearColor Color1;
			FLinearColor Color2;
			uint32 CheckerSize = 0;
		};

		FVector2d GetCurrentDrag() const;
		IImageViewer::FDrawProperties::FPlacement GetPlacementProperties(FIntPoint ImageSize, FVector2d ScaledViewportSize) const;
		IImageViewer::FDrawProperties::FMip GetMipProperties() const;

		void CreateOrDestroyCheckerTextureIfSettingsChanged(const SImageViewport::FDrawSettings& DrawSettings);
		bool MouseIsOverABComparisonDivider(FIntPoint MousePos) const;

		FVector2d GetScaledViewportSize() const;

		FGetImageSize GetImageSize;
		FDrawImage DrawImage;
		FGetDrawSettings GetDrawSettings;
		FGetLayoutAndDPIScaleFactor GetLayoutAndDPIScaleFactor;
		FGetLayoutScaleFactor GetLayoutScaleFactor;
		SImageViewport::FControllerSettings::FOnInputKey OnInputKey;
		const FImageABComparison* ABComparison;

		bool bDragging = false;
		FIntPoint DraggingStart;

		bool bMipSelected = true;
		int32 MipLevel = INDEX_NONE;

		IImageViewer::FDrawProperties::FPlacement CachedPlacement;
		bool bCachedPlacementIsValid = false;

		FImageViewportController Controller;

		TStrongObjectPtr<UTexture2D> CheckerTexture;
		FCheckerTextureSettings CachedCheckerTextureSettings;

		FVector2d CachedScaledViewportSize = FVector2d::ZeroVector;
		bool bZoomOnResize = false;

		double ABComparisonDivider = 0.5;
		bool bDraggingABComparisonDivider = false;
	};
}
