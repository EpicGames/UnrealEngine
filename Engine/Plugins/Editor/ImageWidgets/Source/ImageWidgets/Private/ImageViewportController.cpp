// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageViewportController.h"

namespace UE::ImageWidgets
{
	namespace ImageViewportController_Local
	{
		const TArray<double>& GetZoomLevels()
		{
			constexpr double ZoomMin = 1.0 / 64;
			constexpr double ZoomMax = 64.0;
			constexpr int32 ZoomStepsToDouble = 8;
			constexpr double ZoomPowerIncrement = 1.0 / ZoomStepsToDouble;

			static TArray<double> ZoomLevels = []
			{
				const int32 ZoomMinPower = FMath::RoundToInt32(FMath::Log2(ZoomMin));
				const int32 ZoomMaxPower = FMath::RoundToInt32(FMath::Log2(ZoomMax));
				const int32 Num = (ZoomMaxPower - ZoomMinPower) * ZoomStepsToDouble + 1;

				TArray<double> Result;
				Result.Reserve(Num);

				for (double Power = ZoomMinPower; Power <= ZoomMaxPower; Power += ZoomPowerIncrement)
				{
					Result.Add(FMath::Pow(2.0, Power));
				}

				return Result;
			}();

			return ZoomLevels;
		}

		double ZoomIn(const double CurrentZoom)
		{
			const TArray<double>& ZoomLevels = GetZoomLevels();
			const int32 Index = Algo::LowerBound(ZoomLevels, CurrentZoom);

			if (Index >= ZoomLevels.Num())
			{
				return ZoomLevels.Last();
			}

			if (CurrentZoom < ZoomLevels[Index])
			{
				return ZoomLevels[Index];
			}

			return ZoomLevels[FMath::Min(Index + 1, ZoomLevels.Num() - 1)];
		}

		double ZoomOut(const double CurrentZoom)
		{
			const TArray<double>& ZoomLevels = GetZoomLevels();
			const int32 Index = Algo::LowerBound(ZoomLevels, CurrentZoom);

			if (Index >= ZoomLevels.Num())
			{
				return ZoomLevels.Last();
			}

			if (CurrentZoom > ZoomLevels[Index])
			{
				return ZoomLevels[Index];
			}

			return ZoomLevels[FMath::Max(0, Index - 1)];
		}

		FVector2d AdjustPan(const FVector2d OldPan, const double OldZoom, const double NewZoom, const FVector2d CursorPos, const FVector2d& ImageSize)
		{
			const FVector2d ImageCenter = ImageSize * 0.5;
			const FVector2d DistanceCursorToCenter = CursorPos - ImageCenter;
			const FVector2d DeltaDistanceCursorToCenter = DistanceCursorToCenter * (NewZoom - OldZoom);
			const FVector2d NewPan = OldPan - DeltaDistanceCursorToCenter;

			return NewPan;
		}
	}

	FImageViewportController::FImageViewportController(const EZoomMode DefaultZoomMode)
		: DefaultZoomMode(DefaultZoomMode)
	{
		Reset(FIntPoint::ZeroValue, FVector2d::Zero());
	}

	void FImageViewportController::Pan(const FVector2d ViewportDeltaWithDPIScaling)
	{
		PanAmount += ViewportDeltaWithDPIScaling;
	}

	void FImageViewportController::Reset(const FIntPoint ImageSize, const FVector2d ViewportSizeWithDPIScaling)
	{
		SetZoom(DefaultZoomMode, 1.0, ImageSize, ViewportSizeWithDPIScaling);

		PanAmount = FVector2d::Zero();
	}

	void FImageViewportController::ZoomIn(const FVector2d CursorPixelPos, const FIntPoint ImageSize)
	{
		const double OldZoom = ZoomSettings.Zoom;

		ZoomSettings.Mode = EZoomMode::Custom;
		ZoomSettings.Zoom = ImageViewportController_Local::ZoomIn(ZoomSettings.Zoom);

		PanAmount = ImageViewportController_Local::AdjustPan(PanAmount, OldZoom, ZoomSettings.Zoom, CursorPixelPos, ImageSize);
	}

	void FImageViewportController::ZoomOut(const FVector2d CursorPixelPos, const FIntPoint ImageSize)
	{
		const double OldZoom = ZoomSettings.Zoom;

		ZoomSettings.Mode = EZoomMode::Custom;
		ZoomSettings.Zoom = ImageViewportController_Local::ZoomOut(ZoomSettings.Zoom);

		PanAmount = ImageViewportController_Local::AdjustPan(PanAmount, OldZoom, ZoomSettings.Zoom, CursorPixelPos, ImageSize);
	}

	FVector2d FImageViewportController::GetPan(const FVector2d Drag) const
	{
		return PanAmount + Drag;
	}

	FImageViewportController::FZoomSettings FImageViewportController::GetZoom() const
	{
		return ZoomSettings;
	}

	void FImageViewportController::SetZoom(const EZoomMode ZoomMode, const double Zoom, const FIntPoint ImageSize, const FVector2d ViewportSizeWithDPIScaling)
	{
		ZoomSettings.Mode = ZoomMode;

		if (ZoomSettings.Mode == EZoomMode::Custom)
		{
			ZoomSettings.Zoom = Zoom;
			return;
		}

		if (ImageSize.X <= 0 || ImageSize.Y <= 0 || ViewportSizeWithDPIScaling == FVector2d::Zero())
		{
			ZoomSettings.Zoom = 1.0;
			return;
		}

		const FVector2d SizeRatio = ViewportSizeWithDPIScaling / ImageSize;
		const double SizeRatioMin = SizeRatio.GetMin();

		if (ZoomSettings.Mode == EZoomMode::Fill)
		{
			ZoomSettings.Zoom = SizeRatioMin;
		}
		else
		{
			ZoomSettings.Zoom = FMath::Min(1.0, SizeRatioMin);
		}

		PanAmount = FVector2d::Zero();
	}
}
