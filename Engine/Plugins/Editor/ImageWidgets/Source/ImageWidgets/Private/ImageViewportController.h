// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SImageViewport.h"

namespace UE::ImageWidgets
{
	/**
	 * Camera controller for the 2D viewport supporting panning and zooming.
	 *
	 * Note: All function parameters relating to the viewport need to be passed in with DPI scaling taken into account!
	 */
	class FImageViewportController
	{
	public:
		enum class EZoomMode
		{
			Fit = static_cast<int32>(SImageViewport::FControllerSettings::EDefaultZoomMode::Fit),
			Fill = static_cast<int32>(SImageViewport::FControllerSettings::EDefaultZoomMode::Fill),
			Custom
		};
		static_assert(EZoomMode::Fit != EZoomMode::Fill && EZoomMode::Fit != EZoomMode::Custom && EZoomMode::Fill != EZoomMode::Custom);

		struct FZoomSettings
		{
			EZoomMode Mode;
			double Zoom;
		};

		explicit FImageViewportController(EZoomMode DefaultZoomMode);

		void Pan(FVector2d ViewportDeltaWithDPIScaling);
		void Reset(FIntPoint ImageSize, FVector2d ViewportSizeWithDPIScaling);
		void ZoomIn(FVector2d CursorPixelPos, FIntPoint ImageSize);
		void ZoomOut(FVector2d CursorPixelPos, FIntPoint ImageSize);

		FVector2d GetPan(FVector2d Drag) const;
		FZoomSettings GetZoom() const;
		void SetZoom(EZoomMode ZoomMode, double Zoom, FIntPoint ImageSize, FVector2d ViewportSizeWithDPIScaling);

	private:
		FVector2d PanAmount;
		FZoomSettings ZoomSettings;
		EZoomMode DefaultZoomMode;
	};
}
