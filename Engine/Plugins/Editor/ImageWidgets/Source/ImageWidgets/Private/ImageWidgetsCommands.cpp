// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWidgetsCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ImageWidgetsCommands"

namespace UE::ImageWidgets
{
	FImageWidgetsCommands::FImageWidgetsCommands()
		: TCommands("ImageWidgets", LOCTEXT("ContextDescription", "Image Widgets"), NAME_None, FAppStyle::Get().GetStyleSetName())
	{
	}

	void FImageWidgetsCommands::RegisterCommands()
	{
		UI_COMMAND(MipMinus, "Lower", "Show next lower mip", EUserInterfaceActionType::None, FInputChord());
		UI_COMMAND(MipPlus, "Higher", "Show next higher mip", EUserInterfaceActionType::None, FInputChord());

		UI_COMMAND(ToggleOverlay, "Toggle Overlay", "Toggles the visibility of the toolbar and the status bar.", EUserInterfaceActionType::ToggleButton,
		           FInputChord(EKeys::O));

		UI_COMMAND(Zoom12, "12.5%", "Set Zoom to 12.5%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom25, "25%", "Set Zoom to 25%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom50, "50%", "Set Zoom to 50%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom100, "100%", "Set Zoom to 100%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom200, "200%", "Set Zoom to 200%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom400, "400%", "Set Zoom to 400%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Zoom800, "800%", "Set Zoom to 800%", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ZoomFit, "Scale to Fit", "Optionally downscales the image to fit within the viewport", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(ZoomFill, "Scale to Fill", "Downscales or upscales the image to fill the viewport", EUserInterfaceActionType::RadioButton, FInputChord());
	}
}

#undef LOCTEXT_NAMESPACE
