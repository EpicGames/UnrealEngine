// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"

namespace UE::ImageWidgets
{
	/**
	 * Provides commands used for the image widgets.
	 */
	class FImageWidgetsCommands : public TCommands<FImageWidgetsCommands>
	{
	public:
		FImageWidgetsCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> MipMinus;
		TSharedPtr<FUICommandInfo> MipPlus;

		TSharedPtr<FUICommandInfo> ToggleOverlay;

		TSharedPtr<FUICommandInfo> Zoom12;
		TSharedPtr<FUICommandInfo> Zoom25;
		TSharedPtr<FUICommandInfo> Zoom50;
		TSharedPtr<FUICommandInfo> Zoom100;
		TSharedPtr<FUICommandInfo> Zoom200;
		TSharedPtr<FUICommandInfo> Zoom400;
		TSharedPtr<FUICommandInfo> Zoom800;
		TSharedPtr<FUICommandInfo> ZoomFit;
		TSharedPtr<FUICommandInfo> ZoomFill;
	};
}
