// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

namespace UE::ImageWidgets
{
	DECLARE_LOG_CATEGORY_EXTERN(LogImageWidgets, Log, All)
}
