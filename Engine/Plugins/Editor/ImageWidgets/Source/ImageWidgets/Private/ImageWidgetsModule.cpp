// Copyright Epic Games, Inc. All Rights Reserved.

#if !IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, ImageWidgets);

#else // !IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE

#include "CoreMinimal.h"
#include "ColorViewerSample/ColorViewerCommands.h"
#include "ColorViewerSample/ColorViewerWidget.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ImageWidgetsModule"

namespace UE::ImageWidgets
{
	/**
	 * This module provides the color viewer sample within a tab widget accessible via the "Tools > Miscellaneous" menu.
	 */
	class FImageWidgetsModule : public IModuleInterface
	{
	public:
		void RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup);
		void UnregisterTabSpawners();

		// IModuleInterface overrides - begin
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		// IModuleInterface overrides - end

	private:
		TSharedRef<SDockTab> MakeColorViewerTab(const FSpawnTabArgs&);
		TSharedRef<SWidget> GetColorViewer();

		bool bHasRegisteredTabSpawners = false;
		TWeakPtr<Sample::SColorViewerWidget> ColorViewerPtr;
	};

	void FImageWidgetsModule::RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
	{
		if (bHasRegisteredTabSpawners)
		{
			UnregisterTabSpawners();
		}

		bHasRegisteredTabSpawners = true;

		FTabSpawnerEntry& ColorViewerSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("ColorViewer",
			FOnSpawnTab::CreateRaw(this, &FImageWidgetsModule::MakeColorViewerTab))
		.SetDisplayName(LOCTEXT("ColorViewerTitle", "Color Viewer Sample"))
		.SetTooltipText(LOCTEXT("ColorViewerTooltipText", "Open the Color Viewer tab, a sample application for the ImageWidgets plugin."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsMiscCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Color"));

		if (WorkspaceGroup.IsValid())
		{
			ColorViewerSpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
		}
	}

	void FImageWidgetsModule::UnregisterTabSpawners()
	{
		bHasRegisteredTabSpawners = false;

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("ColorViewer");
	}

	void FImageWidgetsModule::StartupModule()
	{
		bHasRegisteredTabSpawners = false;
		RegisterTabSpawners(nullptr);

		Sample::FColorViewerCommands::Register();
	}

	void FImageWidgetsModule::ShutdownModule()
	{
		Sample::FColorViewerCommands::Unregister();

		UnregisterTabSpawners();
	}

	TSharedRef<SDockTab> FImageWidgetsModule::MakeColorViewerTab(const FSpawnTabArgs&)
	{
		TSharedRef<SDockTab> Tab = SNew(SDockTab)
			.TabRole(NomadTab);
		Tab->SetContent(GetColorViewer());
		return Tab;
	}

	TSharedRef<SWidget> FImageWidgetsModule::GetColorViewer()
	{
		TSharedPtr<Sample::SColorViewerWidget> ColorViewer = ColorViewerPtr.Pin();

		if (!ColorViewer.IsValid())
		{
			ColorViewer = SNew(Sample::SColorViewerWidget);
			ColorViewerPtr = ColorViewer;
		}

		return ColorViewer.ToSharedRef();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::ImageWidgets::FImageWidgetsModule, ImageWidgets)

#endif // !IMAGE_WIDGETS_BUILD_COLOR_VIEWER_SAMPLE
