// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWidgetsStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

namespace UE::ImageWidgets
{
	FName FImageWidgetsStyle::StyleName("ImageWidgetsStyle");

	FImageWidgetsStyle::FImageWidgetsStyle()
		: FSlateStyleSet(StyleName)
	{
		FSlateStyleSet::SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		Set("RichTextBlock.Red", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(1.0f, 0.1f, 0.1f)));
		Set("RichTextBlock.Green", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(0.1f, 1.0f, 0.1f)));
		Set("RichTextBlock.Blue", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(0.1f, 0.1f, 1.0f)));

		FSlateBrush* BrushTableRowOdd = new FSlateBrush();
		BrushTableRowOdd->TintColor = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
		Set("TableRowOdd", BrushTableRowOdd);

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FImageWidgetsStyle::~FImageWidgetsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FImageWidgetsStyle& FImageWidgetsStyle::Get()
	{
		static FImageWidgetsStyle Instance;
		return Instance;
	}
}
