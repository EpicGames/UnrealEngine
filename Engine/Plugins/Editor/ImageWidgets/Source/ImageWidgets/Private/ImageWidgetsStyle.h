// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::ImageWidgets
{
	/**
	 * Style declarations for the image widgets.
	 */
	class FImageWidgetsStyle : public FSlateStyleSet
	{
	public:
		static FName StyleName;

		static FImageWidgetsStyle& Get();

	private:
		FImageWidgetsStyle();
		virtual ~FImageWidgetsStyle() override;
	};
}
