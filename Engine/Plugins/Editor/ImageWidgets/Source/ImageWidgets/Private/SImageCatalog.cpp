// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageCatalog.h"

#include "Algo/Accumulate.h"
#include "Brushes/SlateColorBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"

#include "SImageCatalogItem.h"

#define LOCTEXT_NAMESPACE "SImageViewerCatalog"

namespace UE::ImageWidgets
{
FImageCatalogItemData::FImageCatalogItemData(const FGuid Guid, const FSlateBrush& Brush, const FText& Name, const FText& Info, const FText& ToolTip)
	: Guid(Guid), Thumbnail(Brush), Name(Name), Info(Info), ToolTip(ToolTip)
{
}

using FItemType = TSharedPtr<FImageCatalogItemData>;

struct FGroup
{
	FGroup(FName Name, FText Heading)
		: Name(Name), Heading(MoveTemp(Heading))
	{}

	FName Name;
	FText Heading;
	TSharedPtr<SListView<FItemType>> ListView;
	TArray<FItemType> Items;
	bool bIsExpanded = true;
};

struct FItemLookup
{
	int32 GroupIndex;
	int32 ItemIndex;
};

class FImpl
{
public:
	FImpl(FName DefaultGroupName, const FText& DefaultGroupHeading, ESelectionMode::Type SelectionMode, bool bAllowSelectionAcrossGroups, bool bShowEmptyGroups,
	      const SImageCatalog::FOnItemSelected& OnItemSelected, const SImageCatalog::FOnGetGroupContextMenu& OnGetGroupContextMenu,
	      const SImageCatalog::FOnGetItemsContextMenu& OnGetItemsContextMenu, TSharedPtr<SScrollBox> Layout);

	FName GetDefaultGroupName() const { return DefaultGroupName; }
	bool AddGroup(FName Name, const FText& Heading, const FName* BeforeGroupWithThisName);
	TOptional<TArray<FGuid>> RemoveGroup(FName Name, const FName* GroupToMoveItemsInto);
	int32 NumGroups() const;
	bool SetGroupHeading(FName Name, const FText& Heading);
	TOptional<FName> GetGroupNameAt(int32 Index) const;

	bool AddItem(const FItemType& Item, const FName* GroupName, const FGuid* BeforeItemWithThisGuid);
	bool MoveItem(const FGuid& Guid, const FName* GroupName, const FGuid* BeforeItemWithThisGuid);
	bool RemoveItem(const FGuid& Guid);
	TSharedPtr<const FImageCatalogItemData> GetItem(const FGuid& Guid) const;
	TOptional<int32> GetItemIndex(const FGuid& Guid) const;
	TOptional<TTuple<FName, int32>> GetItemGroupNameAndIndex(const FGuid& Guid) const;
	TSharedPtr<const FImageCatalogItemData> GetItemAt(int32 Index, const FName* GroupName) const;
	TOptional<FGuid> GetItemGuidAt(int32 Index, const FName* GroupName) const;
	bool UpdateItem(const FGuid& Guid, const FSlateBrush* Thumbnail, const FText* Name, const FText* Info, const FText* ToolTip);

	bool SelectItem(const FGuid& Guid, bool bSelected);
	bool ClearSelection(const FName* GroupName);

	int32 NumItems(const FName* GroupName) const;
	TOptional<FName> GetItemGroupName(const FGuid& Guid) const;

private:
	bool AddGroup(FName Name, const FText& Heading, int32 Index);

	const FGroup* FindGroup(FName Name) const;
	FGroup* FindGroup(FName Name)
	{
		return const_cast<FGroup*>(const_cast<const FImpl*>(this)->FindGroup(Name));
	}

	const FItemLookup* FindLookup(const FGuid& Guid) const;
	FItemLookup* FindLookup(const FGuid& Guid)
	{
		return const_cast<FItemLookup*>(const_cast<const FImpl*>(this)->FindLookup(Guid));
	}

	TSharedPtr<SWidget> OnGroupContextMenuOpening(FName Name) const;
	TSharedPtr<SWidget> OnItemsContextMenuOpening() const;
	void SortSelection(TArray<FGuid>& Selection) const;
	void UpdateGroupMapping(int32 StartingIndex);
	void UpdateItemMapping(TArray<FItemType>& Items, int32 StartingIndex);

	const FName DefaultGroupName;
	const FText DefaultGroupHeading;
	const ESelectionMode::Type SelectionMode;
	const bool bAllowSelectionAcrossGroups;
	const bool bShowEmptyGroups;
	SImageCatalog::FOnItemSelected OnItemSelected;
	SImageCatalog::FOnGetGroupContextMenu OnGetGroupContextMenu;
	SImageCatalog::FOnGetItemsContextMenu OnGetItemsContextMenu;
	TSharedPtr<SScrollBox> Layout;
	TArray<TUniquePtr<FGroup>> Groups;
	TMap<FName, int32> GroupMapping;
	TMap<FGuid, FItemLookup> ItemMapping;
};

FImpl::FImpl(const FName DefaultGroupName, const FText& DefaultGroupHeading, const ESelectionMode::Type SelectionMode, const bool bAllowSelectionAcrossGroups,
             const bool bShowEmptyGroups, const SImageCatalog::FOnItemSelected& OnItemSelected,
             const SImageCatalog::FOnGetGroupContextMenu& OnGetGroupContextMenu, const SImageCatalog::FOnGetItemsContextMenu& OnGetItemsContextMenu,
             TSharedPtr<SScrollBox> Layout)
	: DefaultGroupName(DefaultGroupName)
	, DefaultGroupHeading(DefaultGroupHeading)
	, SelectionMode(SelectionMode)
	, bAllowSelectionAcrossGroups(bAllowSelectionAcrossGroups)
	, bShowEmptyGroups(bShowEmptyGroups)
	, OnItemSelected(OnItemSelected)
	, OnGetGroupContextMenu(OnGetGroupContextMenu)
	, OnGetItemsContextMenu(OnGetItemsContextMenu)
	, Layout(MoveTemp(Layout))
{
	AddGroup(DefaultGroupName, DefaultGroupHeading, nullptr);
}

bool FImpl::AddGroup(const FName Name, const FText& Heading, const FName* BeforeGroupWithThisName)
{
	if (Layout.IsValid())
	{
		if (BeforeGroupWithThisName != nullptr)
		{
			if (const int32 *const Index = GroupMapping.Find(*BeforeGroupWithThisName))
			{
				return AddGroup(Name, Heading, *Index);
			}
		}

		return AddGroup(Name, Heading, Groups.Num());
	}
	return false;
}

TOptional<TArray<FGuid>> FImpl::RemoveGroup(const FName Name, const FName* GroupToMoveItemsInto)
{
	if (Name == DefaultGroupName)
	{
		return {};
	}

	if (GroupToMoveItemsInto && Name == *GroupToMoveItemsInto)
	{
		return {};
	}

	if (const int32* GroupIndexPtr = GroupMapping.Find(Name))
	{
		const int32 GroupIndex = *GroupIndexPtr;
		check(0 <= GroupIndex && GroupIndex < Groups.Num());
		FGroup& Group = *Groups[GroupIndex];
		TArray<FItemType>& GroupItems = Group.Items;

		TArray<FGuid> AffectedGuids;
		AffectedGuids.Reserve(GroupItems.Num());

		const int32* NewGroupIndex = GroupToMoveItemsInto ? GroupMapping.Find(*GroupToMoveItemsInto) : nullptr;
		if (GroupToMoveItemsInto && !NewGroupIndex)
		{
			NewGroupIndex = GroupMapping.Find(DefaultGroupName);
		}
		if (NewGroupIndex)
		{
			check(0 <= *NewGroupIndex && *NewGroupIndex < Groups.Num());
			FGroup& NewGroup = *Groups[*NewGroupIndex];
			TArray<FItemType>& NewGroupItems = NewGroup.Items;

			for (FItemType& Item : GroupItems)
			{
				AffectedGuids.Add(Item->Guid);

				FItemLookup* const Lookup = FindLookup(Item->Guid);
				check(Lookup);
				Lookup->GroupIndex = *NewGroupIndex;
				Lookup->ItemIndex = NewGroupItems.Num();

				NewGroupItems.Emplace(MoveTemp(Item));
			}

			NewGroup.ListView->RequestListRefresh();
		}
		else
		{
			for (const FItemType& Item : GroupItems)
			{
				AffectedGuids.Add(Item->Guid);
				ItemMapping.Remove(Item->Guid);
			}
		}

		check(Layout->GetChildren() && GroupIndex < Layout->NumSlots());
		const TSharedRef<SWidget>& WidgetToRemove = Layout->GetSlot(GroupIndex).GetWidget();
		Layout->RemoveSlot(WidgetToRemove);
		Groups.RemoveAt(GroupIndex);
		GroupMapping.Remove(Name);
		UpdateGroupMapping(GroupIndex);

		return {AffectedGuids};
	}

	return {};
}

int32 FImpl::NumGroups() const
{
	return Groups.Num(); 
}

bool FImpl::SetGroupHeading(const FName Name, const FText& Heading)
{
	if (FGroup *const Group = FindGroup(Name))
	{
		Group->Heading = Heading;
		return true;
	}

	return false;
}

TOptional<FName> FImpl::GetGroupNameAt(const int32 Index) const
{
	if (0 <= Index && Index < Groups.Num())
	{
		return {Groups[Index]->Name};
	}

	return {};
}

bool FImpl::AddItem(const FItemType& Item, const FName* GroupName, const FGuid* BeforeItemWithThisGuid)
{
	if (GroupName == nullptr)
	{
		GroupName = &DefaultGroupName;
	}

	if (const int32* GroupIndex = GroupMapping.Find(*GroupName))
	{
		check(0 <= *GroupIndex && *GroupIndex < Groups.Num());
		FGroup& Group = *Groups[*GroupIndex];

		const int32 ItemIndex = [&Item, BeforeItemWithThisGuid, &Group, this]
		{
			if (BeforeItemWithThisGuid)
			{
				if (FItemLookup* LookupData = FindLookup(*BeforeItemWithThisGuid))
				{
					if (Group.Name == Groups[LookupData->GroupIndex]->Name)
					{
						// Set index for added item.
						const int32 Index = LookupData->ItemIndex;

						// Add item at the new index.
						Group.Items.EmplaceAt(Index, Item);

						// Increase index for the item we push back.
						++LookupData->ItemIndex;

						// Update all lookup data for items that come after the item we used to determine the insert location.
						// This way we save the effort for finding the same lookup data again.
						UpdateItemMapping(Group.Items, LookupData->ItemIndex + 1);

						// Tell the outside where the new item was added.
						return Index;
					}
				}
			}

			return Group.Items.Emplace(Item);
		}();

		ItemMapping.Emplace(Item->Guid, {*GroupIndex, ItemIndex});

		Group.ListView->RequestListRefresh();

		return true;
	}

	return false;
}

bool FImpl::MoveItem(const FGuid& Guid, const FName* GroupName, const FGuid* BeforeItemWithThisGuid)
{
	// Do not try to move an item before itself.
	if (BeforeItemWithThisGuid && *BeforeItemWithThisGuid == Guid)
	{
		return false;
	}

	if (FItemLookup *const Lookup = FindLookup(Guid))
	{
		FGroup& GroupFrom = *Groups[Lookup->GroupIndex];

		const int32 *const GroupToIndex = GroupName == nullptr ? &Lookup->GroupIndex : GroupMapping.Find(*GroupName);
		if (GroupToIndex)
		{
			check(0 <= *GroupToIndex && *GroupToIndex < Groups.Num());
			FGroup& GroupTo = *Groups[*GroupToIndex];

			if (BeforeItemWithThisGuid)
			{
				if (FItemLookup *const LookupBefore = FindLookup(*BeforeItemWithThisGuid))
				{
					// Only move if BeforeItems is in the correct group and if item isn't already in the correct position.
					if (*GroupToIndex == LookupBefore->GroupIndex &&
						(Lookup->GroupIndex != LookupBefore->GroupIndex || Lookup->ItemIndex + 1 != LookupBefore->ItemIndex))
					{
						FItemType Item = MoveTemp(GroupFrom.Items[Lookup->ItemIndex]);

						GroupFrom.Items.RemoveAt(Lookup->ItemIndex, EAllowShrinking::No);
						UpdateItemMapping(GroupFrom.Items, Lookup->ItemIndex);

						GroupTo.Items.EmplaceAt(LookupBefore->ItemIndex, MoveTemp(Item));
						Lookup->GroupIndex = LookupBefore->GroupIndex;
						Lookup->ItemIndex = LookupBefore->ItemIndex;
						++LookupBefore->ItemIndex;
						UpdateItemMapping(GroupTo.Items, LookupBefore->ItemIndex + 1);

						return true;
					}
				}
			}
			else
			{
				// Only move if item is not already in the correct group.
				if (*GroupToIndex != Lookup->GroupIndex)
				{
					FItemType Item = MoveTemp(GroupFrom.Items[Lookup->ItemIndex]);

					GroupFrom.Items.RemoveAt(Lookup->ItemIndex, EAllowShrinking::No);
					UpdateItemMapping(GroupFrom.Items, Lookup->ItemIndex);

					Lookup->GroupIndex = *GroupToIndex;
					Lookup->ItemIndex = GroupTo.Items.Emplace(MoveTemp(Item));

					return true;
				}
			}
		}
	}

	return false;
}

bool FImpl::RemoveItem(const FGuid& Guid)
{
	if (const FItemLookup *const Lookup = FindLookup(Guid))
	{
		FGroup& Group = *Groups[Lookup->GroupIndex];
		Group.Items.RemoveAt(Lookup->ItemIndex, EAllowShrinking::No);

		ItemMapping.Remove(Guid);
		UpdateItemMapping(Group.Items, Lookup->ItemIndex);

		Group.ListView->RequestListRefresh();

		return true;
	}

	return false;
}

TSharedPtr<const FImageCatalogItemData> FImpl::GetItem(const FGuid& Guid) const
{
	if (const FItemLookup* Lookup = FindLookup(Guid))
	{
		return Groups[Lookup->GroupIndex]->Items[Lookup->ItemIndex];
	}

	return {};
}

TOptional<int32> FImpl::GetItemIndex(const FGuid& Guid) const
{
	if (const FItemLookup* Lookup = FindLookup(Guid))
	{
		return Lookup->ItemIndex;
	}

	return {};
}

TOptional<TTuple<FName, int32>> FImpl::GetItemGroupNameAndIndex(const FGuid& Guid) const
{
	if (const FItemLookup *const Lookup = FindLookup(Guid))
	{
		return {MakeTuple(Groups[Lookup->GroupIndex]->Name, Lookup->ItemIndex)};
	}

	return {};
}

TSharedPtr<const FImageCatalogItemData> FImpl::GetItemAt(const int32 Index, const FName* GroupName) const
{
	if (!GroupName)
	{
		GroupName = &DefaultGroupName;
	}

	if (const FGroup* const Group = FindGroup(*GroupName))
	{
		if (0 <= Index && Index < Group->Items.Num())
		{
			return {Group->Items[Index]};
		}
	}
	return {};
}

TOptional<FGuid> FImpl::GetItemGuidAt(int32 Index, const FName* GroupName) const
{
	if (!GroupName)
	{
		GroupName = &DefaultGroupName;
	}

	if (const FGroup* const Group = FindGroup(*GroupName))
	{
		if (0 <= Index && Index < Group->Items.Num())
		{
			return {Group->Items[Index]->Guid};
		}
	}
	return {};
}

bool FImpl::UpdateItem(const FGuid& Guid, const FSlateBrush* Thumbnail, const FText* Name, const FText* Info, const FText* ToolTip)
{
	if (const FItemLookup* Lookup = FindLookup(Guid))
	{
		const FItemType& ItemPtr = Groups[Lookup->GroupIndex]->Items[Lookup->ItemIndex];
		check(ItemPtr.IsValid());

		FImageCatalogItemData& Item = *ItemPtr;

		if (Thumbnail) Item.Thumbnail = *Thumbnail;
		if (Name) Item.Name = *Name;
		if (Info) Item.Info = *Info;
		if (ToolTip) Item.ToolTip = *ToolTip;

		return true;
	}

	return false;
}

bool FImpl::SelectItem(const FGuid& Guid, const bool bSelected)
{
	if (const FItemLookup *const Lookup = FindLookup(Guid))
	{
		const FGroup& Group = *Groups[Lookup->GroupIndex];
		const FItemType& Item = Group.Items[Lookup->ItemIndex];
		Group.ListView->SetItemSelection(Item, bSelected);

		return true;
	}

	return false;
}

bool FImpl::ClearSelection(const FName* GroupName)
{
	if (GroupName)
	{
		if (const FGroup *const Group = FindGroup(*GroupName))
		{
			Group->ListView->ClearSelection();
			return true;
		}

		return false;
	}

	for (const TUniquePtr<FGroup>& Group : Groups)
	{
		Group->ListView->ClearSelection();
	}

	return true;
}

int32 FImpl::NumItems(const FName* GroupName) const
{
	if (GroupName)
	{
		if (const FGroup* Group = FindGroup(*GroupName))
		{
			return Group->Items.Num();
		}

		return 0;
	}

	return Algo::Accumulate(Groups, 0, [](int32 Num, const TUniquePtr<FGroup>& Group) { return Num + Group->Items.Num(); });
}

TOptional<FName> FImpl::GetItemGroupName(const FGuid& Guid) const
{
	if (const FItemLookup* Lookup = FindLookup(Guid))
	{
		return { Groups[Lookup->GroupIndex]->Name };
	}

	return {};
}

TSharedPtr<SWidget> FImpl::OnGroupContextMenuOpening(FName Name) const
{
	if (!OnGetGroupContextMenu.IsBound())
	{
		return SNullWidget::NullWidget;
	}

	return OnGetGroupContextMenu.Execute(Name);
}

TSharedPtr<SWidget> FImpl::OnItemsContextMenuOpening() const
{
	if (!OnGetItemsContextMenu.IsBound())
	{
		return SNullWidget::NullWidget;
	}

	TArray<FGuid> SelectedGuids;
	for (const TUniquePtr<FGroup>& Group : Groups)
	{
		Algo::Transform(Group->ListView->GetSelectedItems(), SelectedGuids, [](const FItemType& Item) { return Item->Guid; });
	}

	if (SelectedGuids.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	SortSelection(SelectedGuids);

	return OnGetItemsContextMenu.Execute(SelectedGuids);
}

bool FImpl::AddGroup(FName Name, const FText& Heading, const int32 Index)
{
	check(0 <= Index && Index <= Groups.Num());

	if (!GroupMapping.Contains(Name))
	{
		FGroup& Group = *Groups.EmplaceAt_GetRef(Index, MakeUnique<FGroup>(Name, Heading));
		GroupMapping.Emplace(Name, Index);
		UpdateGroupMapping(Index + 1);

		const auto GetHeading = [&Group]
		{
			return Group.Heading;
		};

		const auto GetHeadingVisibility = [bShowEmptyGroups = bShowEmptyGroups, &Group]
		{
			return !Group.Heading.IsEmpty() && (bShowEmptyGroups || !Group.Items.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		const auto GetHeadingHeight = [bShowEmptyGroups = bShowEmptyGroups, &Group]
		{
			return !Group.Heading.IsEmpty() && (bShowEmptyGroups || !Group.Items.IsEmpty()) ? 26.0f : 0.0f;
		};

		const auto GetVisibility = [bShowEmptyGroups = bShowEmptyGroups, &Group]
		{
			return bShowEmptyGroups || (Group.bIsExpanded && !Group.Items.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		const auto GenerateItemRow = [](const FItemType& ItemData, const TSharedRef<STableViewBase>& OwnerTable)
		{
			static const FTableRowStyle TableRowStyle = []
			{
				FTableRowStyle Style = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
				Style.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background));
				Style.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover));
				Style.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed));
				Style.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover));
				return Style;
			}();

			TSharedPtr<SImageCatalogItem> ItemWidget;
			SAssignNew(ItemWidget, SImageCatalogItem, ItemData);

			return SNew(STableRow<FItemType>, OwnerTable)
				.Style(&TableRowStyle)
				.ShowSelection(true)
				[
					ItemWidget.ToSharedRef()
				];
		};

		const auto SelectionChanged = [bAllowSelectionAcrossGroups = bAllowSelectionAcrossGroups, &GroupName = Group.Name, &Groups = Groups, &OnItemSelected = OnItemSelected](const FItemType& Item, ESelectInfo::Type)
		{
			// Note that Item might be a nullptr since this callback is also executed when clearing the selection of the list.
			if (Item.IsValid())
			{
				if (!bAllowSelectionAcrossGroups)
				{
					for (const TUniquePtr<FGroup>& Group : Groups)
					{
						if (Group->Name != GroupName)
						{
							if (Group->ListView)
							{
								Group->ListView->ClearSelection();
							}
						}
					}
				}

				OnItemSelected.ExecuteIfBound(Item->Guid);
			}
		};

		const auto OpenItemsContextMenu = [this]
		{
			return OnItemsContextMenuOpening();
		};

		TSharedPtr<SWidget> GroupHeader;

		Layout->InsertSlot(Index)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
			.Padding(0.0f, 0.0f, 0.0f, 1.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(GroupHeader, SBox)
					.MinDesiredHeight_Lambda(GetHeadingHeight)
					.Visibility_Lambda(GetHeadingVisibility)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
						.Padding(0.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							.Padding(2.0f, 0.0f, 0.0f, 0.0f)
							.AutoWidth()
							[
								SNew(SButton)
								.ButtonStyle(FCoreStyle::Get(), "NoBorder")
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.ClickMethod(EButtonClickMethod::MouseDown)
								.OnClicked_Lambda([&bIsExpanded = Group.bIsExpanded]()
								{
									bIsExpanded = !bIsExpanded;
									return FReply::Handled();
								})
								.ContentPadding(0.0f)
								.IsFocusable(false)
								[
									SNew(SImage)
									.Image_Lambda([&bIsExpanded = Group.bIsExpanded]()
									{
										return FAppStyle::Get().GetBrush(bIsExpanded ? "TreeArrow_Expanded" : "TreeArrow_Collapsed");
									})
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								]
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.FillWidth(1.0f)
							.Padding(2.0f, 2.0f, 2.0f, 2.0f)
							[
								SNew(STextBlock)
								.Text_Lambda(GetHeading)
								.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
							]
						]
					]
				]
				+ SVerticalBox::Slot()
				[
					SAssignNew(Group.ListView, SListView<FItemType>)
					.ListItemsSource(&Group.Items)
					.Visibility_Lambda(GetVisibility)
					.ScrollbarVisibility(EVisibility::Collapsed)
					.OnGenerateRow_Lambda(GenerateItemRow)
					.SelectionMode(SelectionMode)
					.ClearSelectionOnClick(false)
					.OnSelectionChanged_Lambda(SelectionChanged)
					.OnContextMenuOpening_Lambda(OpenItemsContextMenu)
				]
			]
		];

		TWeakPtr<SWidget> WeakGroupHeader = GroupHeader;
		GroupHeader->SetOnMouseButtonUp(FPointerEventHandler::CreateLambda([this, Name, WeakGroupHeader](const FGeometry& Geometry, const FPointerEvent& Event)
		{
			if (Event.GetEffectingButton() == EKeys::RightMouseButton)
			{
				if (const TSharedPtr<SWidget> PinnedHeader = WeakGroupHeader.Pin())
				{
					const TSharedPtr<SWidget> MenuContent = OnGroupContextMenuOpening(Name);
					if (MenuContent.IsValid())
					{
						const FWidgetPath WidgetPath = Event.GetEventPath() != nullptr ? *Event.GetEventPath() : FWidgetPath();
						FSlateApplication::Get().PushMenu(PinnedHeader->AsShared(), WidgetPath, MenuContent.ToSharedRef(), Event.GetScreenSpacePosition(),
														  FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
					}

					return FReply::Handled().ReleaseMouseCapture();
				}
			}

			return FReply::Unhandled();
		}));

		return true;
	}

	return false;
}

const FGroup* FImpl::FindGroup(const FName Name) const
{
	if (const int32* Index = GroupMapping.Find(Name))
	{
		check(0 <= *Index && *Index < Groups.Num());
		return Groups[*Index].Get();
	}
	return nullptr;
}

const FItemLookup* FImpl::FindLookup(const FGuid& Guid) const
{
	if (const FItemLookup* Lookup = ItemMapping.Find(Guid))
	{
		check(0 <= Lookup->GroupIndex && Lookup->GroupIndex < Groups.Num());
		check(0 <= Lookup->ItemIndex && Lookup->ItemIndex < Groups[Lookup->GroupIndex]->Items.Num());

		return Lookup;
	}

	return nullptr;
}

void FImpl::SortSelection(TArray<FGuid>& Selection) const
{
	TArray<int32> GroupOffsets;
	GroupOffsets.Reserve(Groups.Num());
	int32 Offset = 0;
	for (const TUniquePtr<FGroup>& Group : Groups)
	{
		GroupOffsets.Add(Offset);
		Offset += Group->Items.Num();
	}

	Algo::Sort(Selection, [&ItemMapping = ItemMapping, &GroupOffsets](const FGuid& A, const FGuid& B)
	{
		const FItemLookup *const LookupA = ItemMapping.Find(A);
		const FItemLookup *const LookupB = ItemMapping.Find(B);

		if (LookupA && !LookupB) return true;
		if (!LookupA && LookupB) return false;
		if (!LookupA && !LookupB) return false;

		const int32 PositionA = GroupOffsets[LookupA->GroupIndex] + LookupA->ItemIndex;
		const int32 PositionB = GroupOffsets[LookupB->GroupIndex] + LookupB->ItemIndex;

		return PositionA < PositionB;
	});
}

void FImpl::UpdateGroupMapping(const int32 StartingIndex)
{
	for (int32 GroupIndex = StartingIndex, NumGroups = Groups.Num(); GroupIndex < NumGroups; ++GroupIndex)
	{
		int32 *const GroupMappingIndex = GroupMapping.Find(Groups[GroupIndex]->Name);
		check(GroupMappingIndex);
		*GroupMappingIndex = GroupIndex;

		const TArray<FItemType>& Items = Groups[GroupIndex]->Items;
		for (int32 ItemIndex = 0, NumItems = Items.Num(); ItemIndex < NumItems; ++ItemIndex)
		{
			FItemLookup *const Lookup = ItemMapping.Find(Items[ItemIndex]->Guid);
			check(Lookup);
			Lookup->GroupIndex = GroupIndex;
		}
	}
}

void FImpl::UpdateItemMapping(TArray<FItemType>& Items, const int32 StartingIndex)
{
	for (int32 Index = StartingIndex, Num = Items.Num(); Index < Num; ++Index)
	{
		FItemLookup *const Lookup = ItemMapping.Find(Items[Index]->Guid);
		check(Lookup);
		Lookup->ItemIndex = Index;
	}
}

void SImageCatalog::Construct(const FArguments& Args)
{
	TSharedPtr<SScrollBox> Layout;

	ChildSlot
	[
		SAssignNew(Layout, SScrollBox)
	];

	Impl = MakePimpl<FImpl>(Args._DefaultGroupName, Args._DefaultGroupHeading, Args._SelectionMode, Args._bAllowSelectionAcrossGroups,
	                        Args._bShowEmptyGroups, Args._OnItemSelected, Args._OnGetGroupContextMenu, Args._OnGetItemsContextMenu, Layout);
}

FName SImageCatalog::GetDefaultGroupName() const
{
	return Impl->GetDefaultGroupName();
}

bool SImageCatalog::AddGroup(const FName Name, const FText& Heading)
{
	return Impl->AddGroup(Name, Heading, nullptr);
}

bool SImageCatalog::AddGroup(const FName Name, const FText& Heading, const FName BeforeGroupWithThisName)
{
	return Impl->AddGroup(Name, Heading, &BeforeGroupWithThisName);
}

TOptional<TArray<FGuid>> SImageCatalog::RemoveGroup(const FName Name)
{
	return Impl->RemoveGroup(Name, nullptr);
}

TOptional<TArray<FGuid>> SImageCatalog::RemoveGroup(const FName Name, const FName GroupToMoveItemsInto)
{
	return Impl->RemoveGroup(Name, &GroupToMoveItemsInto);
}

int32 SImageCatalog::NumGroups() const
{
	return Impl->NumGroups();
}

TOptional<FName> SImageCatalog::GetGroupNameAt(int32 Index) const
{
	return Impl->GetGroupNameAt(Index);
}

bool SImageCatalog::SetGroupHeading(const FName Name, const FText& Heading)
{
	return Impl->SetGroupHeading(Name, Heading);
}

bool SImageCatalog::AddItem(const FItemType& Item)
{
	return Impl->AddItem(Item, nullptr, nullptr);
}

bool SImageCatalog::AddItem(const FItemType& Item, const FGuid& BeforeItemWithThisGuid)
{
	return Impl->AddItem(Item, nullptr, &BeforeItemWithThisGuid);
}

bool SImageCatalog::AddItem(const TSharedPtr<FImageCatalogItemData>& Item, const FName Group)
{
	return Impl->AddItem(Item, &Group, nullptr);
}

bool SImageCatalog::AddItem(const TSharedPtr<FImageCatalogItemData>& Item, const FName Group, const FGuid& BeforeItemWithThisGuid)
{
	return Impl->AddItem(Item, &Group, &BeforeItemWithThisGuid);
}

bool SImageCatalog::MoveItem(const FGuid& Guid, const FGuid& BeforeItemWithThisGuid)
{
	return Impl->MoveItem(Guid, nullptr, &BeforeItemWithThisGuid);
}

bool SImageCatalog::MoveItem(const FGuid& Guid, FName Group)
{
	return Impl->MoveItem(Guid, &Group, nullptr);
}

bool SImageCatalog::MoveItem(const FGuid& Guid, FName Group, const FGuid& BeforeItemWithThisGuid)
{
	return Impl->MoveItem(Guid, &Group, &BeforeItemWithThisGuid);
}

bool SImageCatalog::RemoveItem(const FGuid& Guid)
{
	return Impl->RemoveItem(Guid);
}

TSharedPtr<const FImageCatalogItemData> SImageCatalog::GetItem(const FGuid& Guid) const
{
	return Impl->GetItem(Guid);
}

TOptional<int32> SImageCatalog::GetItemIndex(const FGuid& Guid) const
{
	return Impl->GetItemIndex(Guid);
}

TOptional<TTuple<FName, int32>> SImageCatalog::GetItemGroupNameAndIndex(const FGuid& Guid) const
{
	return Impl->GetItemGroupNameAndIndex(Guid);
}

TSharedPtr<const FImageCatalogItemData> SImageCatalog::GetItemAt(const int32 Index) const
{
	return Impl->GetItemAt(Index, nullptr);
}

TSharedPtr<const FImageCatalogItemData> SImageCatalog::GetItemAt(const int32 Index, const FName Group) const
{
	return Impl->GetItemAt(Index, &Group);
}

TOptional<FGuid> SImageCatalog::GetItemGuidAt(const int32 Index) const
{
	return Impl->GetItemGuidAt(Index, nullptr);
}

TOptional<FGuid> SImageCatalog::GetItemGuidAt(const int32 Index, const FName Group) const 
{
	return Impl->GetItemGuidAt(Index, &Group);
}

int32 SImageCatalog::NumItems() const
{
	return Impl->NumItems(nullptr);
}

int32 SImageCatalog::NumItems(FName Group) const
{
	return Impl->NumItems(&Group);
}

TOptional<FName> SImageCatalog::GetItemGroupName(const FGuid& Guid) const
{
	return Impl->GetItemGroupName(Guid);
}

bool SImageCatalog::SelectItem(const FGuid& Guid)
{
	return Impl->SelectItem(Guid, true);
}

bool SImageCatalog::DeselectItem(const FGuid& Guid)
{
	return Impl->SelectItem(Guid, false);
}

void SImageCatalog::ClearSelection()
{
	Impl->ClearSelection(nullptr);
}

bool SImageCatalog::ClearSelection(FName Group)
{
	return Impl->ClearSelection(&Group);
}

bool SImageCatalog::UpdateItem(const FImageCatalogItemData& Item)
{
	return Impl->UpdateItem(Item.Guid, &Item.Thumbnail, &Item.Name, &Item.Info, &Item.ToolTip);
}

bool SImageCatalog::UpdateItemInfo(const FGuid& Guid, const FText& Info)
{
	return Impl->UpdateItem(Guid, nullptr, nullptr, &Info, nullptr);
}

bool SImageCatalog::UpdateItemName(const FGuid& Guid, const FText& Name)
{
	return Impl->UpdateItem(Guid, nullptr, &Name, nullptr, nullptr);
}

bool SImageCatalog::UpdateItemThumbnail(const FGuid& Guid, const FSlateBrush& Thumbnail)
{
	return Impl->UpdateItem(Guid, &Thumbnail, nullptr, nullptr, nullptr);
}

bool SImageCatalog::UpdateItemToolTip(const FGuid& Guid, const FText& ToolTip)
{
	return Impl->UpdateItem(Guid, nullptr, nullptr, nullptr, &ToolTip);
}
}

#undef LOCTEXT_NAMESPACE
