// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageCatalogItem.h"
#include "SImageCatalog.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SImageViewerCatalogItem"

namespace UE::ImageWidgets
{
void SImageCatalogItem::Construct(const FArguments& InArgs, const TSharedPtr<FImageCatalogItemData>& InItemData)
{
	ItemData = InItemData;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			.Padding(FMargin(2.0f, 2.0f))
			.AutoWidth()
		[
			SNew(SImage)
				.DesiredSizeOverride(FVector2D(32.0f, 32.0f))
				.Image(this, &SImageCatalogItem::GetItemThumbnail)
				.ToolTipText(this, &SImageCatalogItem::GetItemToolTip)
		]
		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(FMargin(1.0f, 1.0f))
			[
				SNew(STextBlock)
					.Text(this, &SImageCatalogItem::GetItemName)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(FMargin(1.0f, 1.0f))
			[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
				[
					SNew(STextBlock)
						.Text(this, &SImageCatalogItem::GetItemInfo)
						.TextStyle(&FAppStyle::Get(), "SmallText.Subdued")
				]
			]
		]
	];
}

FText SImageCatalogItem::GetItemInfo() const
{
	return ItemData->Info;
}

FText SImageCatalogItem::GetItemName() const
{
	return ItemData->Name;
}

const FSlateBrush* SImageCatalogItem::GetItemThumbnail() const
{
	return &ItemData->Thumbnail;
}

FText SImageCatalogItem::GetItemToolTip() const
{
	return ItemData->ToolTip;
}
}

#undef LOCTEXT_NAMESPACE
