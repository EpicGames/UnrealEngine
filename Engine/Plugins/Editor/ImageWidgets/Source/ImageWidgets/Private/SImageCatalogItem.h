// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SImageCatalog.h"

namespace UE::ImageWidgets
{
	struct FImageCatalogItemData;

	/**
	 * Widget for a single item row in the catalog.
	 */
	class SImageCatalogItem : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SImageCatalogItem) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedPtr<FImageCatalogItemData>& InItemData);

	private:
		FText GetItemInfo() const;
		FText GetItemName() const;
		const FSlateBrush* GetItemThumbnail() const;
		FText GetItemToolTip() const;

		TSharedPtr<FImageCatalogItemData> ItemData;
	};
}
