// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageViewport.h"

#include "CanvasTypes.h"
#include "ImageABComparison.h"
#include "ImageViewportClient.h"
#include "ImageViewportController.h"
#include "ImageWidgetsCommands.h"
#include "ImageWidgetsStyle.h"
#include "SImageViewportToolbar.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ImageViewport"

namespace UE::ImageWidgets
{
	class FStatusBarExtension
	{
	public:
		virtual ~FStatusBarExtension() = default;

		FName Hook;
		EExtensionHook::Position HookPosition = EExtensionHook::Before;
		TSharedPtr<FUICommandList> CommandList;
		SImageViewport::FStatusBarExtender::FDelegate Delegate;
	};

	void SImageViewport::FStatusBarExtender::AddExtension(FName ExtensionHook, EExtensionHook::Position HookPosition,
	                                                      const TSharedPtr<FUICommandList>& Commands, const FDelegate& Delegate)
	{
		const TPimplPtr<FStatusBarExtension>& Extension = Extensions.Emplace_GetRef(MakePimpl<FStatusBarExtension>());
		Extension->Hook = ExtensionHook;
		Extension->HookPosition = HookPosition;
		Extension->CommandList = Commands;
		Extension->Delegate = Delegate;
	}

	void SImageViewport::FStatusBarExtender::Apply(FName ExtensionHook, EExtensionHook::Position HookPosition, SHorizontalBox& HorizontalBox) const
	{
		for (const TPimplPtr<FStatusBarExtension>& Extension : Extensions)
		{
			if (Extension->Hook == ExtensionHook && Extension->HookPosition == HookPosition)
			{
				Extension->Delegate.ExecuteIfBound(HorizontalBox);
			}
		}
	}

	SImageViewport::SImageViewport()
		: ABComparison(MakePimpl<FImageABComparison>(
			FImageABComparison::FImageIsValid::CreateLambda([this](const FGuid& Guid) { return ImageViewer->IsValidImage(Guid); }),
			FImageABComparison::FGetCurrentImageGuid::CreateLambda([this] { return ImageViewer->GetCurrentImageInfo().Guid; }),
			FImageABComparison::FGetImageName::CreateLambda([this](const FGuid& Guid) { return ImageViewer->GetImageName(Guid); })))
	{
	}

	SImageViewport::~SImageViewport()
	{
		FImageWidgetsCommands::Unregister();
	}

	void SImageViewport::BindCommands()
	{
		const FImageWidgetsCommands& Commands = FImageWidgetsCommands::Get();

		auto ToggleOverlay = [this]
		{
			ViewportOverlay->SetVisibility(ViewportOverlay->GetVisibility() == EVisibility::Visible ? EVisibility::Hidden : EVisibility::Visible);
		};

		CommandList->MapAction(
			Commands.ToggleOverlay,
			FExecuteAction::CreateLambda(ToggleOverlay)
		);

		// ZOOM

		auto MapZoom = [this](const TSharedPtr<FUICommandInfo>& Command, double ZoomFactor)
		{
			CommandList->MapAction(
				Command, FExecuteAction::CreateSP(ImageViewportClient.ToSharedRef(), &FImageViewportClient::SetZoom,
				                                  FImageViewportController::EZoomMode::Custom, ZoomFactor));
		};

		MapZoom(Commands.Zoom12, 0.125);
		MapZoom(Commands.Zoom25, 0.25);
		MapZoom(Commands.Zoom50, 0.5);
		MapZoom(Commands.Zoom100, 1.0);
		MapZoom(Commands.Zoom200, 2.0);
		MapZoom(Commands.Zoom400, 4.0);
		MapZoom(Commands.Zoom800, 8.0);

		auto MapZoomFitFill = [this](const TSharedPtr<FUICommandInfo>& Command, FImageViewportController::EZoomMode ZoomMode)
		{
			CommandList->MapAction(
				Command,
				FExecuteAction::CreateSP(ImageViewportClient.ToSharedRef(), &FImageViewportClient::SetZoom, ZoomMode, 0.0),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, ZoomMode] { return ImageViewportClient->GetZoom().Mode == ZoomMode; }));
		};

		MapZoomFitFill(Commands.ZoomFit, FImageViewportController::EZoomMode::Fit);
		MapZoomFitFill(Commands.ZoomFill, FImageViewportController::EZoomMode::Fill);

		// MIPS

		CommandList->MapAction(
			Commands.MipMinus,
			FExecuteAction::CreateLambda([this]()
			{
				const int32 CurrentMip = ImageViewportClient->GetMipLevel();

				int32 NumMips = 0;
				if (ImageViewer.IsValid())
				{
					const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
					if (ImageInfo.bIsValid)
					{
						NumMips = ImageInfo.NumMips;
					}
				}

				if (NumMips > 0 && CurrentMip < NumMips - 1)
				{
					ImageViewportClient->SetMipLevel(CurrentMip + 1);
				}
			}));
		CommandList->MapAction(
			Commands.MipPlus,
			FExecuteAction::CreateLambda([this]()
			{
				const int32 CurrentMip = ImageViewportClient->GetMipLevel();
				if (CurrentMip > INDEX_NONE)
				{
					ImageViewportClient->SetMipLevel(CurrentMip - 1);
				}
			}));
	}

	void SImageViewport::Construct(const FArguments& InArgs, const TSharedRef<IImageViewer>& InImageViewer)
	{
		FImageWidgetsCommands::Register();

		ImageViewer = InImageViewer;
		check(ImageViewer.IsValid());

		ToolbarExtender = InArgs._ToolbarExtender;
		ToolbarExtenderStyle = InArgs._ToolbarExtenderStyle;
		ToolbarExtenderStyleName = InArgs._ToolbarExtenderStyleName;

		StatusBarExtender = InArgs._StatusBarExtender;

		DrawSettings = InArgs._DrawSettings;
		OverlaySettings = InArgs._OverlaySettings;
		bABComparisonEnabled = InArgs._bABComparisonEnabled;

		auto GetImageSize = [this]
		{
			if (ImageViewer.IsValid())
			{
				const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
				return ImageInfo.Size;
			}
			return FIntPoint::ZeroValue;
		};

		auto DrawImage = [this](FViewport* Viewport, FCanvas* Canvas, const IImageViewer::FDrawProperties& DrawProperties)
		{
			ImageViewer->DrawCurrentImage(Viewport, Canvas, DrawProperties);
		};

		auto GetDrawSettings = [this]
		{
			return DrawSettings.Get({});
		};

		ImageViewportClient = MakeShareable(new FImageViewportClient(
			StaticCastWeakPtr<SEditorViewport>(AsWeak()),
			FGetImageSize::CreateLambda(GetImageSize),
			FDrawImage::CreateLambda(DrawImage),
			FGetDrawSettings::CreateLambda(GetDrawSettings),
			FGetLayoutAndDPIScaleFactor::CreateRaw(this, &SImageViewport::GetLayoutAndDPIScaleFactor),
			FGetLayoutScaleFactor::CreateRaw(this, &SImageViewport::GetLayoutScaleFactor),
			ABComparison.Get(),
			InArgs._ControllerSettings)
			);

		SEditorViewport::Construct(SEditorViewport::FArguments());
	}

	void SImageViewport::ResetController(FIntPoint ImageSize) const
	{
		ImageViewportClient->ResetController(ImageSize);
	}

	void SImageViewport::ResetMip() const
	{
		const int32 SelectedMip = ImageViewportClient->GetMipLevel();
		if (SelectedMip != INDEX_NONE)
		{
			if (ImageViewer.IsValid())
			{
				const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
				if (ImageInfo.bIsValid)
				{
					if (ImageInfo.NumMips <= SelectedMip)
					{
						ImageViewportClient->SetMipLevel(ImageInfo.NumMips - 1);
					}
				}
			}
		}
	}

	void SImageViewport::ResetZoom(FIntPoint ImageSize) const
	{
		ImageViewportClient->ResetZoom(ImageSize);
	}

	TSharedPtr<SViewportToolBar> SImageViewport::GetParentToolbar() const
	{
		return ImageViewportToolbar;
	}

	SImageViewport::FPixelCoordinatesUnderCursorResult SImageViewport::GetPixelCoordinatesUnderCursor() const
	{
		const auto [PixelCoordsValid, PixelCoords] = ImageViewportClient->GetPixelCoordinatesUnderCursor();
		return {PixelCoordsValid, PixelCoords};
	}

	void SImageViewport::RequestRedraw() const
	{
		const TSharedPtr<FEditorViewportClient> ViewportClient = GetViewportClient();
		if (ViewportClient.IsValid())
		{
			ViewportClient->RedrawRequested(ViewportClient->Viewport);
		}
	}

	TSharedRef<FEditorViewportClient> SImageViewport::MakeEditorViewportClient()
	{
		if (!Client.IsValid())
		{
			Client = ImageViewportClient;
		}

		return Client.ToSharedRef();
	}

	TSharedRef<SWidget> SImageViewport::MakeViewportToolbarOverlay()
	{
		auto HasImage = [this]
		{
			if (ImageViewer.IsValid())
			{
				const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
				return ImageInfo.bIsValid;
			}
			return false;
		};

		auto GetNumMips = [this]
		{
			if (ImageViewer.IsValid())
			{
				const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
				if (ImageInfo.bIsValid)
				{
					return ImageInfo.NumMips;
				}
			}

			return 0;
		};

		auto GetImageGuid = [this]
		{
			return ImageViewer.IsValid() ? ImageViewer->GetCurrentImageInfo().Guid : FGuid();
		};

		auto GetOverlaySettings = [this]
		{
			return OverlaySettings.Get({});
		};

		SAssignNew(ImageViewportToolbar, SImageViewportToolbar, StaticCastSharedPtr<FImageViewportClient>(Client), CommandList,
		           SImageViewportToolbar::FConstructParameters{
			           SImageViewportToolbar::FHasImage::CreateLambda(HasImage),
			           SImageViewportToolbar::FNumMips::CreateLambda(GetNumMips),
			           SImageViewportToolbar::FImageGuid::CreateLambda(GetImageGuid),
			           FGetLayoutAndDPIScaleFactor::CreateRaw(this, &SImageViewport::GetLayoutAndDPIScaleFactor),
			           SImageViewportToolbar::FGetOverlaySettings::CreateLambda(GetOverlaySettings),
			           bABComparisonEnabled ? ABComparison.Get() : nullptr,
			           ToolbarExtender, ToolbarExtenderStyle, ToolbarExtenderStyleName
		           });

		ToolbarExtender.Reset();
		ToolbarExtenderStyle = nullptr;
		ToolbarExtenderStyleName = FName();

		return ImageViewportToolbar.ToSharedRef();
	}

	void SImageViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
	{
		Overlay->AddSlot()
			.VAlign(VAlign_Top)
			[
				MakeViewportToolbarOverlay()
			];

		Overlay->AddSlot()
		       .VAlign(VAlign_Bottom)
		[
			MakeStatusBar(Overlay)
		];
	}

	void SImageViewport::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
	{
		SEditorViewport::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

		AllottedGeometryScale = AllottedGeometry.Scale;
	}

	float SImageViewport::GetDPIScaleFactor() const
	{
		if (const TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)); TopLevelWindow.IsValid())
		{
			return TopLevelWindow->GetDPIScaleFactor();
		}
		return 1.0f;
	}

	float SImageViewport::GetLayoutScaleFactor() const
	{
		const float DPI = GetDPIScaleFactor();
		if (DPI > SMALL_NUMBER)
		{
			return FMath::Max(AllottedGeometryScale / DPI, SMALL_NUMBER);
		}
		return 1.0f;
	}

	float SImageViewport::GetLayoutAndDPIScaleFactor() const
	{
		return FMath::Max(AllottedGeometryScale, SMALL_NUMBER);
	}

	FText SImageViewport::GetPickerLabel() const
	{
		if (!ImageViewer.IsValid())
		{
			return {};
		}

		const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
		if (ImageInfo.bIsValid)
		{
			const auto [PixelCoordsValid, PixelCoords] = ImageViewportClient->GetPixelCoordinatesUnderCursor();
			if (PixelCoordsValid)
			{
				const FIntPoint PixelCoordsInt(PixelCoords.X, PixelCoords.Y);
				if (0 <= PixelCoordsInt.X && 0 <= PixelCoordsInt.Y && PixelCoordsInt.X < ImageInfo.Size.X && PixelCoordsInt.Y < ImageInfo.Size.Y)
				{
					const int32 MipIndex = FMath::Max(0, ImageViewportClient->GetMipLevel());

					const FIntPoint MipPixelCoords = [&PixelCoordsInt, MipIndex]
					{
						FIntPoint Coords = PixelCoordsInt;
						if (MipIndex > 0)
						{
							Coords.X /= 1 << MipIndex;
							Coords.Y /= 1 << MipIndex;
						}
						return Coords;
					}();

					const TOptional<TVariant<FColor, FLinearColor>> PixelColor = ImageViewer->GetCurrentImagePixelColor(MipPixelCoords, MipIndex);

					auto CoordsAndColorText = [&MipPixelCoords](const auto* Color, const FNumberFormattingOptions& Formatting)
					{
						return FText::Format(
							LOCTEXT("CoordinatesWithColor", "x={0} y={1}   <RichTextBlock.Red>{2}</> <RichTextBlock.Green>{3}</> <RichTextBlock.Blue>{4}</> {5}"),
							FText::AsNumber(MipPixelCoords.X), FText::AsNumber(MipPixelCoords.Y),
							FText::AsNumber(Color->R, &Formatting), FText::AsNumber(Color->G, &Formatting),
							FText::AsNumber(Color->B, &Formatting), FText::AsNumber(Color->A, &Formatting));
					};

					FNumberFormattingOptions FormattingByte;
					FormattingByte.SetMinimumIntegralDigits(3);

					FNumberFormattingOptions FormattingFloat;
					FormattingFloat.SetMinimumFractionalDigits(3);
					FormattingFloat.SetMaximumFractionalDigits(3);

					if (PixelColor.IsSet())
					{
						const TVariant<FColor, FLinearColor>& PixelColorValue = PixelColor.GetValue();
						if (const FColor* Color = PixelColorValue.TryGet<FColor>())
						{
							return CoordsAndColorText(Color, FormattingByte);
						}
						if (const FLinearColor* ColorLinear = PixelColorValue.TryGet<FLinearColor>())
						{
							return CoordsAndColorText(ColorLinear, FormattingFloat);
						}
					}

					return FText::Format(
						LOCTEXT("CoordinatesOnly", "x={0} y={1}"),
						FText::AsNumber(MipPixelCoords.X), FText::AsNumber(MipPixelCoords.Y));
				}
			}
		}
		return {};
	}

	FText SImageViewport::GetResolutionLabel() const
	{
		if (ImageViewer.IsValid())
		{
			const IImageViewer::FImageInfo ImageInfo = ImageViewer->GetCurrentImageInfo();
			if (ImageInfo.bIsValid)
			{
				FIntPoint Size = ImageInfo.Size;

				const int32 Mip = ImageViewportClient->GetMipLevel();
				if (Mip > 0)
				{
					Size.X = FMath::Max(1, Size.X / (1 << Mip));
					Size.Y = FMath::Max(1, Size.Y / (1 << Mip));
				}

				return FText::Format(LOCTEXT("Resolution", "{0} \u00D7 {1}"), FText::AsNumber(Size.X), FText::AsNumber(Size.Y));
			}
		}
		return {};
	}

	TSharedRef<SWidget> SImageViewport::MakeStatusBar(TSharedRef<SOverlay> Overlay)
	{
		const FMargin SlotPadding(6.0f, 2.0f);

		const TSharedPtr<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		auto ApplyHook = [this, &HorizontalBox](FName ExtensionHook, EExtensionHook::Position HookPosition)
		{
			if (ExtensionHook != NAME_None && StatusBarExtender.IsValid())
			{
				StatusBarExtender->Apply(ExtensionHook, HookPosition, *HorizontalBox);
			}
		};

		ApplyHook(FName("StatusBarLeft"), EExtensionHook::Before);
		{
			HorizontalBox->AddSlot()
			             .Padding(SlotPadding)
			             .AutoWidth()
			             .HAlign(HAlign_Left)
			[
				SNew(STextBlock)
					.Visibility_Lambda([&OverlaySettings = OverlaySettings]
						{
							return OverlaySettings.IsSet() && OverlaySettings.Get().bDisableStatusBarLeft ? EVisibility::Collapsed : EVisibility::Visible;
						})
					.Text(this, &SImageViewport::GetResolutionLabel)
			];
		}
		ApplyHook(FName("StatusBarLeft"), EExtensionHook::After);

		ApplyHook(FName("StatusBarCenter"), EExtensionHook::Before);
		{
			HorizontalBox->AddSlot()
			             .HAlign(HAlign_Center)
			[
				SNullWidget::NullWidget

				// This is deliberately left empty.
				// It allows for adding toolbar extensions though.
			];
		}
		ApplyHook(FName("StatusBarCenter"), EExtensionHook::After);

		ApplyHook(FName("StatusBarRight"), EExtensionHook::Before);
		{
			HorizontalBox->AddSlot()
			             .Padding(SlotPadding)
			             .AutoWidth()
			             .HAlign(HAlign_Right)
			[
				SNew(SRichTextBlock)
					.Visibility_Lambda([&OverlaySettings = OverlaySettings]
						{
							return OverlaySettings.IsSet() && OverlaySettings.Get().bDisableStatusBarRight ? EVisibility::Collapsed : EVisibility::Visible;
						})
					.Text(this, &SImageViewport::GetPickerLabel)
					.DecoratorStyleSet(&FImageWidgetsStyle::Get())
			];
		}
		ApplyHook(FName("StatusBarRight"), EExtensionHook::After);

		StatusBarExtender.Reset();

		return HorizontalBox.ToSharedRef();
	}
}

#undef LOCTEXT_NAMESPACE
