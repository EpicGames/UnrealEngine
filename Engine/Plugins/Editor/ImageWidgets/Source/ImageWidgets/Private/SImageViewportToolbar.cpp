// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImageViewportToolbar.h"

#include "ImageABComparison.h"
#include "ImageViewportClient.h"
#include "ImageWidgetsCommands.h"
#include "SEditorViewportToolBarMenu.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "ImageViewportToolbar"

namespace UE::ImageWidgets
{
	namespace ImageViewportToolbar_Local
	{
		FText Auto = LOCTEXT("Auto", "Auto");
	}

	void SImageViewportToolbar::Construct(const FArguments& InArgs, const TSharedPtr<FImageViewportClient>& InViewportClient,
	                                      const TSharedPtr<FUICommandList>& InCommandList, FConstructParameters Parameters)
	{
		ViewportClient = InViewportClient;
		CommandList = InCommandList;
		check(ViewportClient.IsValid());
		check(CommandList.IsValid());

		HasImage = MoveTemp(Parameters.HasImage);
		NumMips = MoveTemp(Parameters.NumMips);
		ImageGuid = MoveTemp(Parameters.ImageGuid);
		GetLayoutAndDPIScaleFactor = MoveTemp(Parameters.GetLayoutAndDPIScaleFactor);
		GetOverlaySettings = MoveTemp(Parameters.GetOverlaySettings);
		check(HasImage.IsBound());
		check(NumMips.IsBound());
		check(ImageGuid.IsBound());
		check(GetLayoutAndDPIScaleFactor.IsBound());
		check(GetOverlaySettings.IsBound());

		ABComparison = Parameters.ABComparison;

		ToolbarExtenderStyle = Parameters.ToolbarExtenderStyle;
		ToolbarExtenderStyleName = Parameters.ToolbarExtenderStyleName;

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("EditorViewportToolBar.Background"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				  .AutoWidth()
				  .HAlign(HAlign_Left)
				[
					MakeLeftToolbar(Parameters.ToolbarExtender)
				]
				+ SHorizontalBox::Slot()
				  .FillWidth(1.0f)
				  .HAlign(HAlign_Center)
				[
					MakeCenterToolbar(Parameters.ToolbarExtender, ABComparison != nullptr)
				]
				+ SHorizontalBox::Slot()
				  .AutoWidth()
				  .HAlign(HAlign_Right)
				[
					MakeRightToolbar(Parameters.ToolbarExtender)
				]
			]
		];

		SViewportToolBar::Construct(SViewportToolBar::FArguments());
	}

	FSlimHorizontalToolBarBuilder SImageViewportToolbar::GetToolbarBuilder(const TSharedPtr<FExtender>& Extender) const
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None, Extender, true);

		const ISlateStyle* Style = ToolbarExtenderStyle ? ToolbarExtenderStyle : &FAppStyle::Get();
		const FName StyleName = ToolbarExtenderStyle && !ToolbarExtenderStyleName.IsNone() ? ToolbarExtenderStyleName : FName("EditorViewportToolBar");
		ToolbarBuilder.SetStyle(Style, StyleName);

		ToolbarBuilder.SetIsFocusable(false);
		ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

		return ToolbarBuilder;
	}

	TSharedRef<SWidget> SImageViewportToolbar::MakeLeftToolbar(const TSharedPtr<FExtender>& Extender)
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder = GetToolbarBuilder(Extender);

		ToolbarBuilder.BeginSection("ToolbarLeft");
		{
			ToolbarBuilder.BeginBlockGroup();

			ToolbarBuilder.AddWidget(
				SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.Cursor(EMouseCursor::Default)
				.Label(this, &SImageViewportToolbar::GetZoomMenuLabel)
				.OnGetMenuContent(this, &SImageViewportToolbar::MakeZoomMenu)
				.IsEnabled_Lambda([this] { return HasImage.Execute(); })
				.Visibility(this, &SImageViewportToolbar::GetZoomMenuVisibility)
			);

			ToolbarBuilder.AddSeparator();

			ToolbarBuilder.AddWidget(
				SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.Cursor(EMouseCursor::Default)
				.Label(this, &SImageViewportToolbar::GetMipMenuLabel)
				.OnGetMenuContent(this, &SImageViewportToolbar::MakeMipMenu)
				.IsEnabled_Lambda([this] { return HasImage.Execute(); })
				.Visibility(this, &SImageViewportToolbar::GetMipMenuVisibility)
			);

			ToolbarBuilder.EndBlockGroup();
		}
		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SImageViewportToolbar::MakeCenterToolbar(const TSharedPtr<FExtender>& Extender, bool bEnableABComparison)
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder = GetToolbarBuilder(Extender);

		ToolbarBuilder.BeginSection("ToolbarCenter");
		{
			ToolbarBuilder.AddSeparator();

			if (bEnableABComparison && ABComparison)
			{
				auto GetTextButton = [this](const FString& Label, const FCheckBoxStyle* ButtonStyle, FImageABComparison::EAorB AorB)
				{
					return SNew(SCheckBox)
						.Style(ButtonStyle)
						.Visibility(this, &SImageViewportToolbar::GetABVisibility)
						.IsEnabled_Lambda([this, AorB]
							{
								return ABComparison->CanSetABComparison(AorB);
							})
						.IsChecked_Lambda([this, AorB]
							{
								return ABComparison->ABComparisonIsSet(AorB) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
						.OnCheckStateChanged_Lambda([this, AorB](const ECheckBoxState State)
						    {
								ABComparison->SetABComparison(AorB, State != ECheckBoxState::Checked ? FGuid() : ImageGuid.Execute());
						    })
						[
							SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("EditorViewportToolBar.Font"))
								.Text(FText::FromString(Label))
								.ToolTipText_Lambda([&ABComparison = ABComparison, AorB]
									{
										return ABComparison->ABComparisonIsSet(AorB) ? ABComparison->GetName(AorB) : FText();
									})
								.Margin(FMargin(2.0f, 0.0f))
						];
				};

				const FCheckBoxStyle* ButtonStyleStart = &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
				const FCheckBoxStyle* ButtonStyleEnd = &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");

				const TSharedRef<SHorizontalBox> RGBA = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetTextButton("A", ButtonStyleStart, FImageABComparison::EAorB::A)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetTextButton("B", ButtonStyleEnd, FImageABComparison::EAorB::B)
					];

				ToolbarBuilder.AddToolBarWidget(RGBA);
			}

			ToolbarBuilder.AddSeparator();
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.AddSeparator();

		return ToolbarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SImageViewportToolbar::MakeRightToolbar(const TSharedPtr<FExtender>& Extender)
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder = GetToolbarBuilder(Extender);

		ToolbarBuilder.BeginSection("ToolbarRight");
		{
			// This is deliberately left empty.
			// It allows for adding toolbar extensions though.
		}

		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	}

	FText SImageViewportToolbar::GetZoomMenuLabel() const
	{
		const FImageViewportController::FZoomSettings ZoomSettings = ViewportClient->GetZoom();

		const float LayoutAndDPIScaleFactor = GetLayoutAndDPIScaleFactor.Execute();

		const double Zoom = ZoomSettings.Zoom / LayoutAndDPIScaleFactor;
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.SetMaximumFractionalDigits(Zoom < 1.0 ? (Zoom < 0.1 ? 2 : 1) : 0);
		const FText ZoomPercentage = FText::AsPercent(Zoom, &FormattingOptions);

		if (ZoomSettings.Mode == FImageViewportController::EZoomMode::Custom)
		{
			return ZoomPercentage;
		}

		return FText::Format(
			LOCTEXT("ZoomFitFill", "{0} {1}"),
			FText::FromString(ZoomSettings.Mode == FImageViewportController::EZoomMode::Fit ? "Fit" : "Fill"),
			ZoomPercentage);
	}

	TSharedRef<SWidget> SImageViewportToolbar::MakeZoomMenu() const
	{
		FMenuBuilder MenuBuilder(true, CommandList, nullptr, false, &FAppStyle::Get(), false);

		const auto& Commands = FImageWidgetsCommands::Get();

		MenuBuilder.AddMenuEntry(Commands.Zoom12);
		MenuBuilder.AddMenuEntry(Commands.Zoom25);
		MenuBuilder.AddMenuEntry(Commands.Zoom50);
		MenuBuilder.AddMenuEntry(Commands.Zoom100);
		MenuBuilder.AddMenuEntry(Commands.Zoom200);
		MenuBuilder.AddMenuEntry(Commands.Zoom400);
		MenuBuilder.AddMenuEntry(Commands.Zoom800);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(Commands.ZoomFit);
		MenuBuilder.AddMenuEntry(Commands.ZoomFill);

		return MenuBuilder.MakeWidget();
	}

	EVisibility SImageViewportToolbar::GetZoomMenuVisibility() const
	{
		return !GetOverlaySettings.Execute().bDisableZoomButton ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility SImageViewportToolbar::GetMipMenuVisibility() const
	{
		return !GetOverlaySettings.Execute().bDisableMipButton && NumMips.Execute() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility SImageViewportToolbar::GetABVisibility() const
	{
		return !GetOverlaySettings.Execute().bDisableABComparisonButtons ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText SImageViewportToolbar::GetMipMenuLabel() const
	{
		const int32 Mip = ViewportClient->GetMipLevel();
		return FText::Format(LOCTEXT("Mip", "Mip {0}"), Mip == INDEX_NONE ? ImageViewportToolbar_Local::Auto : FText::AsNumber(Mip));
	}

	TSharedRef<SWidget> SImageViewportToolbar::MakeMipMenu() const
	{
		FMenuBuilder MenuBuilder(true, nullptr, nullptr, false, &FAppStyle::Get(), false);

		auto AddMenuEntry = [this, &MenuBuilder](const TAttribute<FText>& Label, const TAttribute<FText>& ToolTip, int32 MipLevel)
		{
			MenuBuilder.AddMenuEntry(
				Label,
				ToolTip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ViewportClient.ToSharedRef(), &FImageViewportClient::SetMipLevel, MipLevel),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, MipLevel]() { return ViewportClient->GetMipLevel() == MipLevel; })
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		};

		AddMenuEntry(ImageViewportToolbar_Local::Auto, LOCTEXT("MipLevelAuto_Tooltip", "Choose Mip Level automatically"), INDEX_NONE);

		if (NumMips.IsBound())
		{
			const int32 Num = NumMips.Execute();
			if (Num > 1)
			{
				MenuBuilder.AddSeparator();

				for (int32 Mip = 0; Mip < Num; ++Mip)
				{
					const FText MipText = FText::AsNumber(Mip);

					AddMenuEntry(FText::Format(LOCTEXT("MipLevel", "Mip {0}"), MipText),
					             FText::Format(LOCTEXT("MipLevel_Tooltip", "Display Mip Level {0}"), MipText),
					             Mip);
				}
			}
		}

		return MenuBuilder.MakeWidget();
	}
}

#undef LOCTEXT_NAMESPACE
