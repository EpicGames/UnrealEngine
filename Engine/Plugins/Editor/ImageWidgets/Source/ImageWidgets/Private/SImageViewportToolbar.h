// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImageViewportClient.h"
#include "SImageViewport.h"
#include "SViewportToolBar.h"

class FExtender;
class FImageViewportCommands;
class FSlimHorizontalToolBarBuilder;
class FUICommandList;
class SViewportToolBarComboMenu;

namespace UE::ImageWidgets
{
	class FImageABComparison;

	/**
	 * Extendable toolbar for the image viewport.
	 */
	class SImageViewportToolbar : public SViewportToolBar
	{
	public:

		/// Callbacks to image viewport to retrieve necessary image metadata.
		DECLARE_DELEGATE_RetVal(bool, FHasImage)
		DECLARE_DELEGATE_RetVal(int32, FNumMips)
		DECLARE_DELEGATE_RetVal(FGuid, FImageGuid)
		DECLARE_DELEGATE_RetVal(SImageViewport::FOverlaySettings, FGetOverlaySettings)

		SLATE_BEGIN_ARGS(SImageViewportToolbar)	{}
		SLATE_END_ARGS()

		/** Parameter for constructing the toolbar. */
		struct FConstructParameters
		{
			FHasImage HasImage;
			FNumMips NumMips;
			FImageGuid ImageGuid;
			FGetLayoutAndDPIScaleFactor GetLayoutAndDPIScaleFactor;
			FGetOverlaySettings GetOverlaySettings;
			FImageABComparison* ABComparison = nullptr;
			TSharedPtr<FExtender> ToolbarExtender;
			const ISlateStyle* ToolbarExtenderStyle = nullptr;
			FName ToolbarExtenderStyleName;
		};

		void Construct(const FArguments& InArgs, const TSharedPtr<FImageViewportClient>& InViewportClient, const TSharedPtr<FUICommandList>& InCommandList,
		               FConstructParameters Parameters);

	private:
		FSlimHorizontalToolBarBuilder GetToolbarBuilder(const TSharedPtr<FExtender>& Extender) const;
		TSharedRef<SWidget> MakeLeftToolbar(const TSharedPtr<FExtender>& Extender);
		TSharedRef<SWidget> MakeCenterToolbar(const TSharedPtr<FExtender>& Extender, bool bEnableABComparison);
		TSharedRef<SWidget> MakeRightToolbar(const TSharedPtr<FExtender>& Extender);

		EVisibility GetZoomMenuVisibility() const;
		EVisibility GetMipMenuVisibility() const;
		EVisibility GetABVisibility() const;
		FText GetMipMenuLabel() const;
		TSharedRef<SWidget> MakeMipMenu() const;

		FText GetZoomMenuLabel() const;
		TSharedRef<SWidget> MakeZoomMenu() const;

		TSharedPtr<FImageViewportClient> ViewportClient;
		TSharedPtr<FUICommandList> CommandList;

		FHasImage HasImage;
		FNumMips NumMips;
		FImageGuid ImageGuid;
		FGetLayoutAndDPIScaleFactor GetLayoutAndDPIScaleFactor;
		FGetOverlaySettings GetOverlaySettings;
		FImageABComparison* ABComparison = nullptr;
		const ISlateStyle* ToolbarExtenderStyle = nullptr;
		FName ToolbarExtenderStyleName;
	};
}
