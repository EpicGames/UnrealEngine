// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TVariant.h"

class FCanvas;
class FViewport;

namespace UE::ImageWidgets
{
	/**
	 * Interface for a component that wants to show image related content with the Slate widgets in the ImageWidget module.
	 * In this context, an "image" is considered to be any 2D content that is contained within an axis-aligned rectangle.
	 */
	class IImageViewer
	{
	public:
		/**
		 * Information about an image to be displayed.
		 */
		struct FImageInfo
		{
			/** Unique image identifier. It can encode any helpful metadata as long as no two provided images have the same GUID. */
			FGuid Guid;

			/** XY size of the image in pixels. */
			FIntPoint Size;

			/** Number of available MIPs. This should be set to zero if the image type does not support mips. */
			int32 NumMips;

			/** Indicates that this image is valid for display. */
			bool bIsValid;
		};

		/**
		 * Information necessary for correctly drawing an image.
		 */
		struct FDrawProperties
		{
			/**
			 * Where in the 2D plane the image rectangle is supposed to be drawn.
			 */
			struct FPlacement
			{
				/** Offset from the origin, i.e. (0, 0). */
				FVector2d Offset;

				/** XY size of the axis aligned rectangle containing the image. */
				FVector2d Size;

				/**
				 * The zoom factor used for the image.
				 * While this might not be necessary for drawing the image, it can be helpful in certain use cases. For example, interpolation could explicitly
				 * be turned off when zooming into a texture to show the discrete pixel contents of the texture instead of the interpolated result.
				 */
				double ZoomFactor;
			};

			/**
			 * Information about MIP levels. This can be ignored if the image type does not support MIPs. 
			 */
			struct FMip
			{
				/** The selected MIP level. */
				float MipLevel;
			};

			/**
			 * Information necessary for rendering AB comparisons.
			 */
			struct FABComparison
			{
				/** @return true if an AB comparison should be drawn instead of a single image. */
				bool IsActive() const { return GuidA.IsValid() && GuidB.IsValid(); };

				/** Unique identifier for image A. */
				FGuid GuidA;
				
				/** Unique identifier for image B. */
				FGuid GuidB;

				/**
				 * Value between 0..1 to indicate where the threshold between images A and B is.
				 * A value of 0 means that only B should be drawn.
				 * A value of 0.5 means that the left half of A and the right half of B should be drawn.
				 * A value of 1 means that only A should be drawn. 
				 */
				double Threshold;
			};

			FPlacement Placement;
			FMip Mip;
			FABComparison ABComparison;
		};

		/**
		 * Provides any necessary metadata for the image widgets about the image that is currently supposed to be displayed. This data is generic in the sense
		 * that the image widgets don't need to know any of the image structure, its content or how to draw it. Instead, the image drawing is done directly
		 * via @see DrawCurrentImage.
		 * @return information about the image to be displayed
		 */
		virtual FImageInfo GetCurrentImageInfo() const = 0;

		/**
		 * Draws the image that is currently supposed to be displayed within the 2D viewport.
		 * @param Viewport The viewport the image is drawn into
		 * @param Canvas The canvas used for drawing the image
		 * @param Properties Information for drawing the image based on the current image viewport state 
		 */
		virtual void DrawCurrentImage(FViewport* Viewport, FCanvas* Canvas, const FDrawProperties& Properties) = 0;
		
		/**
		 * Provides information about a given pixel.
		 * @param PixelCoords XY coordinates for the pixel
		 * @param MipLevel MIP level that is currently displayed; this can be ignored for images not supporting MIPs 
		 * @return Either a color value in byte or float format, i.e. FColor or FLinearColor, or no value if there is no valid pixel at the provided coordinates 
		 */
		virtual TOptional<TVariant<FColor, FLinearColor>> GetCurrentImagePixelColor(FIntPoint PixelCoords, int32 MipLevel) const = 0;

		/**
		 * Notifies about the image with the given GUID being selected.
		 * This can be implemented as an empty function if the image viewer implementation does not support switching between different images.
		 * @param Guid Unique identifier of the selected image
		 */
		virtual void OnImageSelected(const FGuid& Guid) = 0;

		/**
		 * Returns if a given GUID represents a currently available image.
		 * @param Guid Unique identifier of the potentially available image
		 * @return true if the GUID represents a currently available image
		 */
		virtual bool IsValidImage(const FGuid& Guid) const = 0;

		/**
		 * Returns the name of a currently available image. This is potentially used in the UI to refer to an image.
		 * @param Guid Unique identifier of the potentially available image
		 * @return Name of the image if the GUID represents a currently available image
		 */
		virtual FText GetImageName(const FGuid& Guid) const = 0;

	protected:
		~IImageViewer() = default;
	};
}
