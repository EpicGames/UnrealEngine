// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Views/ITypedTableView.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ImageWidgets
{
	/**
	 * Contains all data for a catalog item.
	 */
	struct FImageCatalogItemData
	{
		IMAGEWIDGETS_API FImageCatalogItemData(FGuid Guid, const FSlateBrush& Brush, const FText& Name, const FText& Info, const FText& ToolTip);

		/** Unique identifier for the catalog item */
		FGuid Guid;

		/** Brush used for displaying the item's thumbnail */
		FSlateBrush Thumbnail;

		/** Name of the item */
		FText Name;

		/** Auxiliary information for the item */
		FText Info;

		/** Tooltip that is shown when hovering over any part of the item's widget in the catalog */
		FText ToolTip;
	};

	/**
	 * Generic catalog widget for listing and interacting with 2D image-like content.
	 * Each catalog item is represented by its own widget based on its @see FImageCatalogItemData.
	 * Entries in the catalog can be assigned to customizable groups.
	 */
	class SImageCatalog : public SCompoundWidget
	{
	public:
		/**
		 * Delegate that gets called when an item is selected in the catalog.
		 * The given @see FGuid identifies the item that was selected.
		 */
		DECLARE_DELEGATE_OneParam(FOnItemSelected, const FGuid&)

		/**
		 * Delegate that gets called for creating a context menu for a group.
		 * Return @see SNullWidget::NullWidget to not show a context menu.
		 */
		DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetGroupContextMenu, FName)

		/**
		 * Delegate that gets called for creating a context menu for a set of selected items.
		 * Return @see SNullWidget::NullWidget to not show a context menu.
		 */
		DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetItemsContextMenu, const TArray<FGuid>&)

		SLATE_BEGIN_ARGS(SImageCatalog)
				: _DefaultGroupName(NAME_None)
				, _DefaultGroupHeading(FText())
				, _SelectionMode(ESelectionMode::Multi)
				, _bAllowSelectionAcrossGroups(false)
				, _bShowEmptyGroups(false)
			{
			}

			/** Name of the default group, i.e. the group that gets used when no group is explicitly specified. */
			SLATE_ARGUMENT(FName, DefaultGroupName)

			/** Header text for the default group. */
			SLATE_ARGUMENT(FText, DefaultGroupHeading)

			/** Delegate that gets called when an item is selected in the catalog. */
			SLATE_EVENT(FOnItemSelected, OnItemSelected)

			/** Delegate that gets called for creating a context menu for a group. */
			SLATE_EVENT(FOnGetGroupContextMenu, OnGetGroupContextMenu)

			/** Delegate that gets called for creating a context menu for a set of selected items. */
			SLATE_EVENT(FOnGetItemsContextMenu, OnGetItemsContextMenu)

			/** Defines the selection behavior within an item list, e.g. only allow single item selection or do not allow any selection. */
			SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)

			/** When an item is selected, the selection in other groups will be cleared unless this flag is set to true. */
			SLATE_ARGUMENT(bool, bAllowSelectionAcrossGroups)

			/** Empty groups will be hidden unless this flag is set to true. */
			SLATE_ARGUMENT(bool, bShowEmptyGroups)
		SLATE_END_ARGS()

		/**
		 * Function used by Slate to construct the image catalog widget with the given arguments.
		 * @param Args Slate arguments defined above
		 */
		IMAGEWIDGETS_API void Construct(const FArguments& Args);

		/**
		 * @returns The name of the default group.
		 */
		IMAGEWIDGETS_API FName GetDefaultGroupName() const;

		/**
		 * Add a custom group to the catalog.
		 * @param Name Unique identifier for the group.
		 * @param Heading Header text to be used for the group; no header will be shown if this is empty.
		 * @returns False if a group with the given name already exits, otherwise true.
		 */
		IMAGEWIDGETS_API bool AddGroup(FName Name, const FText& Heading);

		/**
		 * Add a custom group to the catalog before an already existing group.
		 * @param Name Unique identifier for the group.
		 * @param Heading Header text to be used for the group; no header will be shown if this is empty.
		 * @param BeforeGroupWithThisName Name of an already existing group before which the new custom group is going to be added.
		 * @returns False if a group with the given name already exits or if the group before which the new group is supposed to be added does not exist,
		 * otherwise true.
		 */
		IMAGEWIDGETS_API bool AddGroup(FName Name, const FText& Heading, FName BeforeGroupWithThisName);

		/**
		 * Set the header text for an existing group.
		 * @param Name Unique identifier for the group.
		 * @param Heading Header text to be used for the group; no header will be shown if this is empty.
		 * @returns False if a group with the given name does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool SetGroupHeading(FName Name, const FText& Heading);

		/**
		* Remove an existing group and any items in the group.
		* @param Name Unique identifier of the group; must not be the default group name.
		* @returns An empty value if a group with the given name does not exist, otherwise a list of unique identifiers for the items that were in the group
		* and got removed from the catalog.
		*/
		IMAGEWIDGETS_API TOptional<TArray<FGuid>> RemoveGroup(FName Name);

		/**
		* Remove an existing group, and moves any items in the group into another group.
		* Note that if the other group does not exist, the items get moved into the default group instead.
		* @param Name Unique identifier of the group; must not be the default group name.
		* @param GroupToMoveItemsInto Name of an already existing group in which the items from the removed group get moved into.
		* @returns An empty value if a group with the given name does not exist, otherwise a list of unique identifiers for the items that were in the group
		* and got moved into the other group.
		*/
		IMAGEWIDGETS_API TOptional<TArray<FGuid>> RemoveGroup(FName Name, FName GroupToMoveItemsInto);

		/**
		 * Return the number of groups in the catalog including the default group.
		 * @returns The number of groups in the catalog.
		 */
		IMAGEWIDGETS_API int32 NumGroups() const;

		/**
		 * Return the name of the group at a given position.
		 * @param Index Position of the group, with 0 being the first, and @see NumGroups() - 1 being the last.
		 * @returns An empty value if the given position is invalid, otherwise the name of the group at that position.
		 */
		IMAGEWIDGETS_API TOptional<FName> GetGroupNameAt(int32 Index) const;

		/**
		 * Add an item to the default group.
		 * @param Item Data for the item that is being added.
		 * @returns True if the item was added successfully.
		 */
		IMAGEWIDGETS_API bool AddItem(const TSharedPtr<FImageCatalogItemData>& Item);

		/**
		 * Add an item to the default group right before an existing item.
		 * If the unique identifier for the other item is invalid or not in the default group, the item will be added at the end of the default group instead.
		 * @param Item Data for the item that is being added.
		 * @param BeforeItemWithThisGuid Unique identifier of the other item before which the new item should be added.
		 * @returns True if the item was added successfully.
		 */
		IMAGEWIDGETS_API bool AddItem(const TSharedPtr<FImageCatalogItemData>& Item, const FGuid& BeforeItemWithThisGuid);

		/**
		 * Add an item to an existing group.
		 * @param Item Data for the item that should be added.
		 * @param Group Name of the existing group to which the item should be added. 
		 * @returns False if a group with the given name does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool AddItem(const TSharedPtr<FImageCatalogItemData>& Item, FName Group);

		/**
		 * Add an item to an existing group right before an existing item.
		 * If the unique identifier for the other item is invalid or not in the given group, the item will be added at the end of the group instead.
		 * @param Item Data for the item that should be added.
		 * @param Group Name of the existing group to which the item should be added.
		 * @param BeforeItemWithThisGuid Unique identifier of the other item before which the new item should be added.
		 * @returns False if a group with the given name does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool AddItem(const TSharedPtr<FImageCatalogItemData>& Item, FName Group, const FGuid& BeforeItemWithThisGuid);

		/**
		 * Move an already existing item before another item within the same group.
		 * @param Guid The unique identifier of the existing item that should be moved.
		 * @param BeforeItemWithThisGuid Unique identifier of the other item before which the item should be moved.
		 * @returns False if the item to be moved or the item before which it should be moved does not exist or if both are not in the same group,
		 * otherwise true.
		 */
		IMAGEWIDGETS_API bool MoveItem(const FGuid& Guid, const FGuid& BeforeItemWithThisGuid);

		/**
		 * Move an already existing item to another group.
		 * @param Guid The unique identifier of the existing item that should be moved.
		 * @param Group Name of the existing group to which the item should be moved.
		 * @returns False if an item with the given unique identifier does not exist, a group with the given name does not exist, or the item is already in the
		 * given group, otherwise true.
		 */
		IMAGEWIDGETS_API bool MoveItem(const FGuid& Guid, FName Group);

		/**
		 * Move an already existing item to another group, before another item within that group.
		 * @param Guid The unique identifier of the existing item that should be moved.
		 * @param Group Name of the existing group to which the item should be moved.
		 * @param BeforeItemWithThisGuid Unique identifier of the other item before which the item should be moved.
		 * @returns False if an item with the given unique identifier does not exist, a group with the given name does not exist, or the item is already in the
		 * given group, otherwise true.
		 */
		IMAGEWIDGETS_API bool MoveItem(const FGuid& Guid, FName Group, const FGuid& BeforeItemWithThisGuid);

		/**
		 * Remove an existing item.
		 * @param Guid The unique identifier of the existing item.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool RemoveItem(const FGuid& Guid);

		/**
		 * Return the total number of items in the catalog.
		 * @returns The number of items in the catalog across all groups.
		 */
		IMAGEWIDGETS_API int32 NumItems() const;

		/**
		 * Return the number of items in a group.
		 * @param Group Name of an existing group.
		 * @returns The number of items in the given group.
		 */
		IMAGEWIDGETS_API int32 NumItems(FName Group) const;

		/**
		 * Retrieve the existing item for a given unique identifier.
		 * @param Guid The unique identifier of the item.
		 * @return The pointer to the item or an invalid pointer if no item with the given unique identifier exists.
		 */
		IMAGEWIDGETS_API TSharedPtr<const FImageCatalogItemData> GetItem(const FGuid& Guid) const;

		/**
		 * Return the group an existing item belongs to.
		 * @param Guid The unique identifier of the existing item.
		 * @returns An empty value if an item with the given unique identifier does not exist, otherwise the name of the group the item belongs to.
		 */
		IMAGEWIDGETS_API TOptional<FName> GetItemGroupName(const FGuid& Guid) const;

		/**
		 * Return the index of an existing item within the group it belongs to.
		 * @param Guid The unique identifier of the existing item.
		 * @returns An empty value if no item with the given unique identifier exists, otherwise index of the item in the group.
		 */
		IMAGEWIDGETS_API TOptional<int32> GetItemIndex(const FGuid& Guid) const;

		/**
		 * Return an existing item's group and the index within that group.
		 * @param Guid The unique identifier of the existing item.
		 * @returns An empty value if an item with the given unique identifier does not exist, otherwise the name of the group the item belongs to and
		 * the index of the item in the group.
		 */
		IMAGEWIDGETS_API TOptional<TTuple<FName, int32>> GetItemGroupNameAndIndex(const FGuid& Guid) const;

		/**
		 * Retrieve the existing item for a given index within the default group.
		 * @param Index Index of the item in the default group; should be at least 0 and less than @see NumItems(@see GetDefaultGroup()).
		 * @returns The pointer to the item or an invalid pointer if the index is invalid.
		 */
		IMAGEWIDGETS_API TSharedPtr<const FImageCatalogItemData> GetItemAt(int32 Index) const;

		/**
		* Retrieve the existing item within a given group for a given index.
		 * @param Index Index of the item in the given group; should be at least 0 and less than @see NumItems(Group).
		 * @param Group Name of an existing group.
		 * @returns The pointer to the item or an invalid pointer if the group does not exist or the index is invalid.
		 */
		IMAGEWIDGETS_API TSharedPtr<const FImageCatalogItemData> GetItemAt(int32 Index, FName Group) const;

		/**
		 * Retrieve the unique identifier of an existing item at the given index within the default group.
		 * @param Index Index of the item in the default group; should be at least 0 and less than @see NumItems(@see GetDefaultGroup()).
		 * @return Unique identifier of the item or no value if the index is invalid.
		 */
		IMAGEWIDGETS_API TOptional<FGuid> GetItemGuidAt(int32 Index) const;

		/**
		 * Retrieve the unique identifier of an existing item at the given index within the given group.
		 * @param Index Index of the item in the given group; should be at least 0 and less than @see NumItems(Group).
		 * @param Group Name of an existing group.
		 * @return Unique identifier of the item or no value if the group does not exist or the index is invalid.
		 */
		IMAGEWIDGETS_API TOptional<FGuid> GetItemGuidAt(int32 Index, FName Group) const;

		/**
		 * Select an existing item.
		 * @param Guid The unique identifier of the existing item.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool SelectItem(const FGuid& Guid);

		/**
		 * Deselect an existing item.
		 * @param Guid The unique identifier of the existing item.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool DeselectItem(const FGuid& Guid);

		/**
		 * Clear any selection in the catalog, i.e. across all groups.
		 */
		IMAGEWIDGETS_API void ClearSelection();

		/**
		 * Clear the selection for a given group; selections in other groups will be unchanged.
		 * @param Group Name of an existing group.
		 * @returns False if a group with the given name does not exist
		 */
		IMAGEWIDGETS_API bool ClearSelection(FName Group);

		/**
		 * Update an existing item's data. The item data needs to contain the item's unique identifier.
		 * @param Item Data for the exiting item that should be updated, including its unique identifier.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool UpdateItem(const FImageCatalogItemData& Item);

		/**
		 * Update the info text of an existing item.
		 * @param Guid The unique identifier of the existing item.
		 * @param Info Text for the info label in the item's widget.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool UpdateItemInfo(const FGuid& Guid, const FText& Info);

		/**
		 * Update the name of an existing item.
		 * Nothing happens if no item with the given unique identifier exists in the catalog.
		 * @param Guid The unique identifier of the existing item.
		 * @param Name Text for the name label in the item's widget.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool UpdateItemName(const FGuid& Guid, const FText& Name);

		/**
		 * Update the thumbnail of an existing item.
		 * Nothing happens if no item with the given unique identifier exists in the catalog.
		 * @param Guid The unique identifier of the existing item.
		 * @param Thumbnail Brush used for the thumbnail in the item's widget.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool UpdateItemThumbnail(const FGuid& Guid, const FSlateBrush& Thumbnail);

		/**
		 * Update the tooltip text of an existing item.
		 * Nothing happens if no item with the given unique identifier exists in the catalog.
		 * @param Guid The unique identifier of the existing item.
		 * @param ToolTip Text for the tooltip label in the item's widget.
		 * @returns False if an item with the given unique identifier does not exist, otherwise true.
		 */
		IMAGEWIDGETS_API bool UpdateItemToolTip(const FGuid& Guid, const FText& ToolTip);

	private:
		TPimplPtr<class FImpl> Impl;
	};
}
