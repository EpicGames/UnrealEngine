// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

class SViewportToolBar;

namespace UE::ImageWidgets
{
	class FImageABComparison;
	class FImageViewportClient;
	class FStatusBarExtension;
	class IImageViewer;
	class SImageViewportToolbar;

	/**
	 * Generic viewport for displaying and interacting with 2D image-like content.
	 * The drawing of the images is deferred to an @see IImageViewer implementation that needs to be provided upon construction. This viewport only uses the
	 * metadata provided by the image viewer to have sufficient information about the image without being aware of its actual format or contents.
	 */
	class SImageViewport : public SEditorViewport
	{
	public:
		/**
		 * Contains any extensions widgets for the viewport toolbar.
		 * This implementation is similar to the existing toolbar extender, i.e. FExtender, which is designed to only work with dedicated toolbar builders and
		 * can currently not be used with status bars.
		 */
		class FStatusBarExtender
		{
		public:
			/**
			 * Delegate that gets called for each extension when the status bar is constructed.
			 * The given @see SHorizontalBox is the layout container that the widget for the extension needs to be added to by calling
			 * @see SHorizontalBox::AddSlot().
			 */
			DECLARE_DELEGATE_OneParam(FDelegate, SHorizontalBox&);

			/**
			 * Adds an extension to the status bar.
			 * @param ExtensionHook The section the extension is applied to
			 * @param HookPosition Where in relation to the section the extension is applied to
			 * @param Commands The UI command list responsible for handling actions used in the widgets added in the extension
			 * @param Delegate Delegate called for adding the extension widgets when the status bar is constructed
			 */
			IMAGEWIDGETS_API void AddExtension(FName ExtensionHook, EExtensionHook::Position HookPosition, const TSharedPtr<FUICommandList>& Commands,
			                                   const FDelegate& Delegate);

		private:
			friend SImageViewport;

			/** Used by the viewport to add extensions to the status bar. */
			void Apply(FName ExtensionHook, EExtensionHook::Position HookPosition, SHorizontalBox& HorizontalBox) const;

			/** List of extensions that get applied to the viewport status bar. */
			TArray<TPimplPtr<FStatusBarExtension>> Extensions;
		};

		/**
		 * Settings related to drawing viewport contents other than the image itself.
		 */
		struct FDrawSettings
		{
			/** Clear color for the viewport. */
			FLinearColor ClearColor = FLinearColor::Black;

			/** Flag that enables drawing a border around the image. If enabled, the border is drawn underneath the image.
			 *  The center of the line drawn for the border is at the exact edge of the image, i.e. when the image is drawn on top of it, the outer half of the
			 *  border is visible, and the inner half is occluded by the image. */
			bool bBorderEnabled = false;

			/** Thickness of the border. */
			float BorderThickness = 1.0f;

			/** Color of the border */
			FLinearColor BorderColor = FVector3f(0.2f);

			/** Flag that enables the drawing of the background color within the image rectangle. */
			bool bBackgroundColorEnabled = false;

			/** Color for the background within the image rectangle. If this is different to the clear color, it shows where the image is even if nothing is drawn. */
			FLinearColor BackgroundColor = FLinearColor::Black;

			/** Flag that enabled the drawing of a background checker texture. If this is enabled, the texture is drawn on top of the background color. */
			bool bBackgroundCheckerEnabled = false;

			/** First color used in the background checker texture. */
			FLinearColor BackgroundCheckerColor1 = FLinearColor::White;

			/** Second color used in the background checker texture. */
			FLinearColor BackgroundCheckerColor2 = FVector3f(0.8f);

			/** Size of a single square within the background checker texture in pixels. */
			uint32 BackgroundCheckerSize = 8;
		};

		/**
		 * Settings related to viewport controls.
		 */
		struct FControllerSettings
		{
			enum class EDefaultZoomMode
			{
				Fit, // Make the image fit within the viewport, but do not make it larger than the original size.
				Fill // Make the image fit within the viewport, and if it is smaller than the viewport, zoom in to fill the viewport.
			};

			/**
			 * Delegate for custom input key event handling in the viewport client.
			 * The delegate is called after the image viewports input key events were handled, e.g. zoom or view reset via F key, and before a call
			 * to @see FEditorViewportClient::InputKey().
			 *
			 * @param	EventArgs - The Input event args.
			 * @return	True to consume the key event, false to pass it on.
			 */
			DECLARE_DELEGATE_RetVal_OneParam(bool, FOnInputKey, const FInputKeyEventArgs& EventArgs);

			/** Zoom mode that gets set on viewport construction and whenever the viewport controller is reset via @see ResetController(). */
			EDefaultZoomMode DefaultZoomMode = EDefaultZoomMode::Fit;

			/** Delegate for custom input key handling. */
			FOnInputKey OnInputKey;

			/** Adjusts zoom to be proportional to viewport resizes, e.g. doubling the viewport size will double the zoom. */
			bool bZoomOnResize = false;
		};

		/**
		 * Settings related to the viewport overlay.
		 */
		struct FOverlaySettings
		{
			/** Do not show the zoom button in the left toolbar. */
			bool bDisableZoomButton = false;

			/** Do not show the MIP button in the left toolbar. */
			bool bDisableMipButton = false;

			/** Do not show the AB comparison buttons in the center toolbar. */
			bool bDisableABComparisonButtons = false;

			/** Do not show the bottom left status bar. */
			bool bDisableStatusBarLeft = false;

			/** Do not show the bottom right status bar. */
			bool bDisableStatusBarRight = false;
		};

		SLATE_BEGIN_ARGS(SImageViewport)
			: _ToolbarExtenderStyle(nullptr)
			, _bABComparisonEnabled(false)
			{
			}

			/** Extensions for the viewport toolbar; valid hooks are "ToolbarLeft", "ToolbarCenter", "ToolbarRight" */
			SLATE_ARGUMENT(TSharedPtr<FExtender>, ToolbarExtender)

			/** Style override for viewport toolbar extensions. */
			SLATE_ARGUMENT(const ISlateStyle*, ToolbarExtenderStyle)

			/**
			 * Name used for the viewport toolbar extensions style override.
			 * Only used if ToolbarExtenderStyle is set. If this is not set, "EditorViewportToolBar" will be used.
			 */
			SLATE_ARGUMENT(FName, ToolbarExtenderStyleName)

			/** Extensions for the viewport status bar; valid hooks are "StatusBarLeft", "StatusBarCenter", "StatusBarRight" */
			SLATE_ARGUMENT(TSharedPtr<FStatusBarExtender>, StatusBarExtender)

			/** Settings for drawing viewport contents other than the actual image */
			SLATE_ATTRIBUTE(FDrawSettings, DrawSettings)

			/** Settings for customizing the viewport overlay */
			SLATE_ATTRIBUTE(FOverlaySettings, OverlaySettings)

			/** Enables AB comparison controls in the toolbar */
			SLATE_ARGUMENT(bool, bABComparisonEnabled)

			/** Settings for controlling the viewport */
			SLATE_ARGUMENT(FControllerSettings, ControllerSettings)

		SLATE_END_ARGS()

		IMAGEWIDGETS_API SImageViewport();
		IMAGEWIDGETS_API virtual ~SImageViewport() override;

		/**
		 * Function used by Slate to construct the image viewport widget with the given arguments.
		 * @param InArgs Slate arguments defined above
		 * @param InImageViewer @see IImageViewer implementation that holds and draws the actual image contents.
		 */
		IMAGEWIDGETS_API void Construct(const FArguments& InArgs, const TSharedRef<IImageViewer>& InImageViewer);

		/**
		 * Provides access to the viewport toolbar, which is needed to dynamically generate certain toolbar widgets, e.g. an @see SEditorViewportToolbarMenu,
		 * as part of a toolbar extension. 
		 * @return Pointer to the viewport toolbar 
		 */
		IMAGEWIDGETS_API TSharedPtr<SViewportToolBar> GetParentToolbar() const;

		/**
		 * Result for calls to @see GetPixelCoordinatesUnderCursor.
		 */
		struct FPixelCoordinatesUnderCursorResult
		{
			/** Indicates that the cursor position is currently valid. This is set to false, for example, when the cursor is outside the widget. */
			bool bIsValid = false;

			/** Pixel coordinates under the cursor relative to the image rectangle size and placement. Note that the coordinates might be outside the image
			 * rectangle, i.e. values might be negative or larger than the image size. */
			FVector2d Coordinates;
		};

		/**
		 * Provides the pixel coordinates under the cursor.
		 * @return Pixel coordinates result under the cursor; see @see FPixelCoordinatesUnderCursorResult for more details
		 */
		IMAGEWIDGETS_API FPixelCoordinatesUnderCursorResult GetPixelCoordinatesUnderCursor() const;

		/**
		 * Request to redraw the viewport contents as soon as possible.
		 */
		IMAGEWIDGETS_API void RequestRedraw() const;

		/**
		 * Resets the camera controller to default values.
		 * @param ImageSize Size of the currently displayed image in pixels
		 */
		IMAGEWIDGETS_API void ResetController(FIntPoint ImageSize) const;

		/**
		 * Resets the MIP level to the default.
		 */
		IMAGEWIDGETS_API void ResetMip() const;

		/**
		 * Resets the zoom to default.
		 * @param ImageSize Size of the currently displayed image in pixels
		 */
		IMAGEWIDGETS_API void ResetZoom(FIntPoint ImageSize) const;

		// SEditorViewport overrides - begin
	public:
		virtual bool IsVisible() const override { return true; } // Overriding this method avoids the 0.25 seconds update delay in SEditorViewport.
	protected:
		virtual void BindCommands() override;
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
		virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
		// SEditorViewport overrides - end

		// SCompoundWidget overrides - begin
		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
		// SCompoundWidget overrides - end

	private:
		TSharedRef<SWidget> MakeViewportToolbarOverlay();

		/** Create the status bar widgets. */
		TSharedRef<SWidget> MakeStatusBar(TSharedRef<SOverlay> Overlay);

		/** Provides the window DPI scale factor. */
		float GetDPIScaleFactor() const;

		/** Provides the external layout factor (ratio of accumulated scale to window DPI). */
		float GetLayoutScaleFactor() const;

		/** Provides the layout and DPI scale factor, i.e. the full accumulated geometry scale. */
		float GetLayoutAndDPIScaleFactor() const;

		/** Provides the text for the color picker display in the status bar. */
		FText GetPickerLabel() const;

		/** Provides the text for the resolution display in the status bar. */
		FText GetResolutionLabel() const;

		/** Makes the draw settings available either as fixed values or via a callback to the outside of the viewport. */
		TAttribute<FDrawSettings> DrawSettings;

		/** Makes the overlay settings available either as fixed values or via a callback to the outside of the viewport. */
		TAttribute<FOverlaySettings> OverlaySettings;

		/** Flag that determines if AB comparison widgets are enabled or not. The value does not change after the call to @see Construct(). */
		bool bABComparisonEnabled = false;

		/** Data and logic related to AB comparisons. This is effectively unused when @see bABComparisonEnabled is set to false. */
		TPimplPtr<FImageABComparison> ABComparison;

		/** The image viewer that holds and draws the actual images. */
		TSharedPtr<IImageViewer> ImageViewer;

		/** The viewport client that takes care of camera controls and displaying the viewport contents. */
		TSharedPtr<FImageViewportClient> ImageViewportClient;

		/** The toolbar that controls some of the behavior of the viewport and optionally also the image viewer via toolbar extensions. */
		TSharedPtr<SImageViewportToolbar> ImageViewportToolbar;

		/** Toolbar extensions provided by the call to @see Construct(). This pointer is reset after the extensions were applied during construction. */
		TSharedPtr<FExtender> ToolbarExtender;

		/** Style override for toolbar extender. Only used during construction, then reset. */
		const ISlateStyle* ToolbarExtenderStyle = nullptr;
		FName ToolbarExtenderStyleName;

		/** Status bar extensions provided by the call to @see Construct(). This pointer is reset after the extensions were applied during construction. */
		TSharedPtr<FStatusBarExtender> StatusBarExtender;

		/** Cached scale for the allotted geometry; used to correctly incorporate the overall scale of the layout when drawing the image. */
		mutable float AllottedGeometryScale = 1.0f;
	};
}
